{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Color Correction Regions (CCR)",
	"Description": "Color correction/shading constrained to regions/volumes",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "ColorCorrectRegions",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "ColorCorrectRegionsEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name": "nDisplayModularFeatures",
			"Enabled": true
		},
		{
			"Name": "ColorGrading",
			"Enabled": true
		},
		{
			"Name": "ObjectMixer",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncClient",
			"Enabled": false,
			"Optional": true
		}
	]
}