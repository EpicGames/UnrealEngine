[CoreRedirects]

+ClassRedirects=(OldName="ColorCorrectRegion_C",NewName="/Script/ColorCorrectRegions.ColorCorrectionRegion",OverrideClassName="/Script/CoreUObject.Class")
+PackageRedirects=(OldName="/ColorCorrectRegions/Blueprints/ColorCorrectRegion",Removed=True)
+ClassRedirects=(OldName="/Script/ColorCorrectRegions.ColorCorrectWindow", NewName="/Script/ColorCorrectRegions.ColorCorrectionWindow")

