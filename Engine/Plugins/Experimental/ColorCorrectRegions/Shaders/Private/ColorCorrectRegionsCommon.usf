// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/DeferredShadingCommon.ush"        // GetGBufferData()


#ifdef NEEDS_SCENE_TEXTURES
#undef NEEDS_SCENE_TEXTURES
#endif
#define NEEDS_SCENE_TEXTURES 1

SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessInput_0)
SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessInput_1)
SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessInput_2)
SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessInput_3)
SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessInput_4)
SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessOutput)

Texture2D PostProcessInput_0_Texture;
Texture2D PostProcessInput_1_Texture;
Texture2D PostProcessInput_2_Texture;
Texture2D PostProcessInput_3_Texture;
Texture2D PostProcessInput_4_Texture;

SamplerState PostProcessInput_0_Sampler;
SamplerState PostProcessInput_1_Sampler;
SamplerState PostProcessInput_2_Sampler;
SamplerState PostProcessInput_3_Sampler;
SamplerState PostProcessInput_4_Sampler;


SamplerState PostProcessInputSampler;
SamplerState PostProcessInput_BilinearSampler;

#define PostProcessInput_0_SharedSampler PostProcessInput_0_Sampler
#define PostProcessInput_1_SharedSampler PostProcessInput_1_Sampler
#define PostProcessInput_2_SharedSampler PostProcessInput_2_Sampler
#define PostProcessInput_3_SharedSampler PostProcessInput_3_Sampler
#define PostProcessInput_4_SharedSampler PostProcessInput_4_Sampler

//////////////////////////////////////////////////////////////////////////
//! Must match ESceneTextureId

#define PPI_SceneColor 0
#define PPI_SceneDepth 1
#define PPI_DiffuseColor 2
#define PPI_SpecularColor 3
#define PPI_SubsurfaceColor 4
#define PPI_BaseColor 5
#define PPI_Specular 6
#define PPI_Metallic 7
#define PPI_WorldNormal 8
#define PPI_SeparateTranslucency 9
#define PPI_Opacity 10
#define PPI_Roughness 11
#define PPI_MaterialAO 12
#define PPI_CustomDepth 13
#define PPI_PostProcessInput0 14
#define PPI_PostProcessInput1 15
#define PPI_PostProcessInput2 16
#define PPI_PostProcessInput3 17
#define PPI_PostProcessInput4 18
#define PPI_PostProcessInput5 19 // (UNUSED)
#define PPI_PostProcessInput6 20 // (UNUSED)
#define PPI_DecalMask 21
#define PPI_ShadingModelColor 22
#define PPI_ShadingModelID 23
#define PPI_AmbientOcclusion 24
#define PPI_CustomStencil 25
#define PPI_StoredBaseColor 26
#define PPI_StoredSpecular 27
#define PPI_Velocity 28
#define PPI_WorldTangent 29
#define PPI_Anisotropy 30
#define PPI_IsFirstPerson 31

//////////////////////////////////////////////////////////////////////////



/**
 * Parameters needed by pixel shader material inputs, related to Geometry.
 * These are independent of vertex factory.
 */
struct FMaterialPixelParameters
{
	/** Normalized world space normal. */
	half3 WorldNormal;

	/** Normalized world space tangent. */
	half3 WorldTangent;

	/** Normalized world space reflected camera vector. */
	half3 ReflectionVector;

	/** Normalized world space camera vector, which is the vector from the point being shaded to the camera position. */
	half3 CameraVector;

	/**
	 * Like SV_Position (.xy is pixel position at pixel center, z:DeviceZ, .w:SceneDepth)
	 * using shader generated value SV_POSITION
	 * Note: this is not relative to the current viewport.  RelativePixelPosition = MaterialParameters.SvPosition.xy - View.ViewRectMin.xy;
	 */
	MaterialFloat4 SvPosition;

	/** Post projection position reconstructed from SvPosition, before the divide by W. left..top -1..1, bottom..top -1..1  within the viewport, W is the SceneDepth */
	MaterialFloat4 ScreenPosition;

	/**
	 * Interpolated worldspace position of this pixel
	 */
	MaterialFloat3 AbsoluteWorldPosition;

	/**
	 * Interpolated worldspace position of this pixel, centered around the camera
	 */
	MaterialFloat3 WorldPosition_CamRelative;
};


/**
 * Parameters calculated from the pixel material inputs.
 */
struct FPixelMaterialInputs
{
	MaterialFloat3 EmissiveColor;
	MaterialFloat Opacity;
	MaterialFloat OpacityMask;
	MaterialFloat3 BaseColor;
	MaterialFloat Metallic;
	MaterialFloat Specular;
	MaterialFloat Roughness;
	MaterialFloat Anisotropy;
	MaterialFloat3 Normal;
	MaterialFloat3 Tangent;
	MaterialFloat4 Subsurface;
	MaterialFloat AmbientOcclusion;
	MaterialFloat2 Refraction;
	MaterialFloat PixelDepthOffset;
	uint ShadingModel;

};


/**
 * Parameters needed by vertex shader material inputs.
 * These are independent of vertex factory.
 */
struct FMaterialVertexParameters
{
	// Position in the translated world (VertexFactoryGetWorldPosition).
	// Previous position in the translated world (VertexFactoryGetPreviousWorldPosition) if
	//    computing material's output for previous frame (See {BasePassVertex,Velocity}Shader.usf).
	MaterialFloat3 WorldPosition;
	// TangentToWorld[2] is WorldVertexNormal
	half3x3 TangentToWorld;

	// If either USE_INSTANCING or (IS_MESHPARTICLE_FACTORY && FEATURE_LEVEL >= FEATURE_LEVEL_SM4)
	// is true, PrevFrameLocalToWorld is a per-instance transform
	MaterialFloat4x4 PrevFrameLocalToWorld;

	MaterialFloat3 PreSkinnedPosition;
	MaterialFloat3 PreSkinnedNormal;

	half4 VertexColor;

	// Index into View.PrimitiveSceneData
	uint PrimitiveId;

};

MaterialFloat3 ComposureColorGradePerRange(MaterialFloat3 ColorAP1, MaterialFloat4 ColorSaturation, MaterialFloat4 ColorContrast, MaterialFloat4 ColorGamma, MaterialFloat4 ColorGain, MaterialFloat4 ColorOffset)
{
	MaterialFloat Luma = dot(ColorAP1, MaterialFloat3(0.2722287168, 0.6740817658, 0.0536895174));
	ColorAP1 = max(0, lerp(Luma.xxx, ColorAP1, ColorSaturation.xyz * ColorSaturation.w));
	ColorAP1 = pow(ColorAP1 * (5.55555555), ColorContrast.xyz * ColorContrast.w) * 0.18;
	ColorAP1 = pow(ColorAP1, 1.0 / (ColorGamma.xyz * ColorGamma.w));
	ColorAP1 = ColorAP1 * (ColorGain.xyz * ColorGain.w) + (ColorOffset.xyz + ColorOffset.w);
	return ColorAP1;
}


/** Calculate a reflection vector about the specified world space normal. Optionally normalize this normal **/
MaterialFloat3 ReflectionAboutCustomWorldNormal(FMaterialPixelParameters Parameters, MaterialFloat3 WorldNormal, bool bNormalizeInputNormal)
{
	if (bNormalizeInputNormal)
	{
		WorldNormal = normalize(WorldNormal);
	}

	return -Parameters.CameraVector + WorldNormal * dot(WorldNormal, Parameters.CameraVector) * 2.0;
}


MaterialFloat3 GetWorldPosition(FMaterialPixelParameters Parameters)
{
	return Parameters.AbsoluteWorldPosition;
}

MaterialFloat2 GetPixelPosition(FMaterialPixelParameters Parameters)
{
	return Parameters.SvPosition.xy - MaterialFloat2(PostProcessOutput_ViewportMin);
}

MaterialFloat2 GetViewportUV(FMaterialPixelParameters Parameters)
{
	return GetPixelPosition(Parameters) * PostProcessOutput_ViewportSizeInverse;
}

half3 GetMaterialNormalRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.Normal;
}

half3 GetMaterialNormal(FMaterialPixelParameters Parameters, FPixelMaterialInputs PixelMaterialInputs)
{
	half3 RetNormal;
	RetNormal = GetMaterialNormalRaw(PixelMaterialInputs);
	return RetNormal;
}

// Returns whether a scene texture id is a for a post process input or not.
bool IsPostProcessInputSceneTexture(const uint SceneTextureId)
{
	return (SceneTextureId >= PPI_PostProcessInput0 && SceneTextureId <= PPI_PostProcessInput6);
}

// Transforms viewport UV to scene texture's UV.
MaterialFloat2 ViewportUVToSceneTextureUV(MaterialFloat2 ViewportUV, const uint SceneTextureId)
{
	if (IsPostProcessInputSceneTexture(SceneTextureId))
	{
		switch (SceneTextureId)
		{
		case PPI_PostProcessInput0:
			return ViewportUV * PostProcessInput_0_UVViewportSize + PostProcessInput_0_UVViewportMin;
		case PPI_PostProcessInput1:
			return ViewportUV * PostProcessInput_1_UVViewportSize + PostProcessInput_1_UVViewportMin;
		case PPI_PostProcessInput2:
			return ViewportUV * PostProcessInput_2_UVViewportSize + PostProcessInput_2_UVViewportMin;
		case PPI_PostProcessInput3:
			return ViewportUV * PostProcessInput_3_UVViewportSize + PostProcessInput_3_UVViewportMin;
		case PPI_PostProcessInput4:
			return ViewportUV * PostProcessInput_4_UVViewportSize + PostProcessInput_4_UVViewportMin;
		default:
			return ViewportUV;
		}
	}

	return ViewportUVToBufferUV(ViewportUV);
}

// Get default scene texture's UV.
MaterialFloat2 GetDefaultSceneTextureUV(FMaterialPixelParameters Parameters, const uint SceneTextureId)
{
	return ViewportUVToSceneTextureUV(GetViewportUV(Parameters), SceneTextureId);
}

/** Rotates Position about the given axis by the given angle, in radians, and returns the offset to Position. */
MaterialFloat3 RotateAboutAxis(MaterialFloat4 NormalizedRotationAxisAndAngle, MaterialFloat3 PositionOnAxis, MaterialFloat3 Position)
{
	// Project Position onto the rotation axis and find the closest point on the axis to Position
	MaterialFloat3 ClosestPointOnAxis = PositionOnAxis + NormalizedRotationAxisAndAngle.xyz * dot(NormalizedRotationAxisAndAngle.xyz, Position - PositionOnAxis);
	// Construct orthogonal axes in the plane of the rotation
	MaterialFloat3 UAxis = Position - ClosestPointOnAxis;
	MaterialFloat3 VAxis = cross(NormalizedRotationAxisAndAngle.xyz, UAxis);
	MaterialFloat CosAngle;
	MaterialFloat SinAngle;
	sincos(NormalizedRotationAxisAndAngle.w, SinAngle, CosAngle);
	// Rotate using the orthogonal axes
	MaterialFloat3 R = UAxis * CosAngle + VAxis * SinAngle;
	// Reconstruct the rotated world space position
	MaterialFloat3 RotatedPosition = ClosestPointOnAxis + R;
	// Convert from position to a position offset
	return RotatedPosition - Position;
}

MaterialFloat3 RotateTranslateScale(MaterialFloat3 OriginalVec, MaterialFloat3 Translation, MaterialFloat3 Rotation, MaterialFloat3 Scale)
{
	// Translate
	MaterialFloat3 Translated = (OriginalVec - Translation);
	// Rotate
	MaterialFloat3 RotatedZ = RotateAboutAxis(MaterialFloat4(MaterialFloat3(0.00000000, 0.00000000, 1.00000000), Rotation.b * -1), MaterialFloat3(0.00000000, 0.00000000, 0.00000000), Translated);
	MaterialFloat3 RotatedTranslatedZ = (RotatedZ + Translated);
	MaterialFloat3 RotatedY = RotateAboutAxis(MaterialFloat4(MaterialFloat3(0.00000000, 1.00000000, 0.00000000), Rotation.g), MaterialFloat3(0.00000000, 0.00000000, 0.00000000), RotatedTranslatedZ);
	MaterialFloat3 RotatedTranslatedYZ = (RotatedY + RotatedTranslatedZ);
	MaterialFloat3 RotatedX = RotateAboutAxis(MaterialFloat4(MaterialFloat3(1.00000000, 0.00000000, 0.00000000), Rotation.r), MaterialFloat3(0.00000000, 0.00000000, 0.00000000), RotatedTranslatedYZ);
	MaterialFloat3 RotatedTranslated = (RotatedX + RotatedTranslatedYZ);
	return (RotatedTranslated / (Scale));
}


#define POST_PROCESS_INPUT(Index)	\
{									\
	if (bFiltered)					\
	{								\
		return Texture2DSample(PostProcessInput_##Index##_Texture, PostProcessInput_BilinearSampler, UV); \
	}								\
	else							\
	{								\
		return Texture2DSample(PostProcessInput_##Index##_Texture, PostProcessInput_##Index##_SharedSampler, UV); \
	}								\
}

/** Applies an offset to the scene texture lookup and decodes the HDR linear space color. */
float4 SceneTextureLookup(float2 UV, int SceneTextureIndex, bool bFiltered)
{
#if SHADING_PATH_DEFERRED
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(UV, false);
#endif
	switch (SceneTextureIndex)
	{
		// order needs to match to ESceneTextureId

	case PPI_SceneColor:
		return float4(CalcSceneColor(UV), 0);
	case PPI_SeparateTranslucency:
		return float4(1, 1, 1, 1);

	case PPI_PostProcessInput0:
		POST_PROCESS_INPUT(0);
	case PPI_PostProcessInput1:
		POST_PROCESS_INPUT(1);
	case PPI_PostProcessInput2:
		POST_PROCESS_INPUT(2);
	case PPI_PostProcessInput3:
		POST_PROCESS_INPUT(3);
	case PPI_PostProcessInput4:
		POST_PROCESS_INPUT(4);
#if SHADING_PATH_DEFERRED
	case PPI_SceneDepth:
		return ScreenSpaceData.GBuffer.Depth;
	case PPI_DiffuseColor:
		return float4(ScreenSpaceData.GBuffer.DiffuseColor, 0);
	case PPI_SpecularColor:
		return float4(ScreenSpaceData.GBuffer.SpecularColor, 0);
	case PPI_SubsurfaceColor:
		return IsSubsurfaceModel(ScreenSpaceData.GBuffer.ShadingModelID) ? float4(ExtractSubsurfaceColor(ScreenSpaceData.GBuffer), ScreenSpaceData.GBuffer.CustomData.a) : ScreenSpaceData.GBuffer.CustomData;
	case PPI_BaseColor:
		return float4(ScreenSpaceData.GBuffer.BaseColor, 0);
	case PPI_Specular:
		return ScreenSpaceData.GBuffer.Specular;
	case PPI_Metallic:
		return ScreenSpaceData.GBuffer.Metallic;
	case PPI_WorldNormal:
		return float4(ScreenSpaceData.GBuffer.WorldNormal, 0);
	case PPI_Opacity:
		return ScreenSpaceData.GBuffer.CustomData.a;
	case PPI_Roughness:
		return ScreenSpaceData.GBuffer.Roughness;
	case PPI_MaterialAO:
		return ScreenSpaceData.GBuffer.GBufferAO;
	case PPI_CustomDepth:
		return ScreenSpaceData.GBuffer.CustomDepth;
	case PPI_ShadingModelColor:
		return float4(GetShadingModelColor(ScreenSpaceData.GBuffer.ShadingModelID), 1);
	case PPI_ShadingModelID:
		return float4(ScreenSpaceData.GBuffer.ShadingModelID, 0, 0, 0);
	case PPI_AmbientOcclusion:
		return ScreenSpaceData.AmbientOcclusion;
	case PPI_CustomStencil:
		return ScreenSpaceData.GBuffer.CustomStencil;
	case PPI_StoredBaseColor:
		return float4(ScreenSpaceData.GBuffer.StoredBaseColor, 0);
	case PPI_StoredSpecular:
		return float4(ScreenSpaceData.GBuffer.StoredSpecular.rrr, 0);

	case PPI_WorldTangent:
		return float4(ScreenSpaceData.GBuffer.WorldTangent, 0);
	case PPI_Anisotropy:
		return ScreenSpaceData.GBuffer.Anisotropy;
	case PPI_IsFirstPerson:
		return IsFirstPerson(ScreenSpaceData.GBuffer) ? 1.0f : 0.0f;
#endif
	default:
		return float4(0, 0, 0, 0);
	}
}

