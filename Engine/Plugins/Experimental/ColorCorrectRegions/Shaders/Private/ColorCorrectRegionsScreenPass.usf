// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

#if !CLEAR_PS
Texture2D InputTexture;
SamplerState InputSampler;
#endif

void MainVS(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	out noperspective float4 OutUVAndScreenPos : TEXCOORD0,
	out float4 OutPosition : SV_POSITION)
{
	DrawRectangle(InPosition, InTexCoord, OutPosition, OutUVAndScreenPos);
}

#if !CLEAR_PS
float4 MainPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0
	) : SV_Target0
{
	float2 UV = UVAndScreenPos.xy;
	return Texture2DSample(InputTexture, InputSampler, UV);
}
#else
float4 MainPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0
	) : SV_Target0
{
	return float4(0.,0.,0.,0.);
}
#endif