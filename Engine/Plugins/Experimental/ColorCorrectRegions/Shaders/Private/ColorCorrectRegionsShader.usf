// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ColorUtils.ush"
#include "/Engine/Private/DistanceField/GlobalDistanceFieldShared.ush"
#include "/Engine/Generated/GeneratedUniformBuffers.ush" 
#include "/Engine/Private/Random.ush"
#include "/Engine/Private/SceneTexturesCommon.ush"
#include "/Engine/Private/SceneData.ush"
#include "/Engine/Private/ScreenPass.ush"
#include "/Engine/Private/SceneTextureParameters.ush"
#include "/Engine/Private/TonemapCommon.ush"

#include "/ColorCorrectRegionsShaders/Private/ColorCorrectRegionsCommon.usf"

//////////////////////////////////////////////////////////////////////////


#define SHAPE_TYPE_SPHERE 0
#define SHAPE_TYPE_BOX 1
#define SHAPE_TYPE_CYLINDER 2
#define SHAPE_TYPE_CONE 3

#define WINDOW_TYPE_PLANE 0
#define WINDOW_TYPE_CIRCLE 1

#define TEMPERATURE_TYPE_LEGACY 0
#define TEMPERATURE_TYPE_WHITE_BALANCE 1
#define TEMPERATURE_TYPE_COLOR_TEMP 2
#define TEMPERATURE_TYPE_SKIP 3

#define CCR_SHADER_DISPLAY_BOUNDING_RECT 0
#define CLIP_PIXELS_OUTSIDE_AABB 1

#define EXCLUDE_STENCIL 0

Texture2D<uint> MergedStencilTexture;

struct FExternalExpressions
{
	float3 Rotate;
	float3 Translate;
	float3 Scale;

	float4 ColorSaturation;
	float4 ColorContrast;
	float4 ColorGamma;
	float4 ColorGain;
	float4 ColorOffset;

#if ADVANCED_CC	
	float4 ColorSaturationShadows;
	float4 ColorContrastShadows;
	float4 ColorGammaShadows;
	float4 ColorGainShadows;
	float4 ColorOffsetShadows;
	float ShadowMax;

	float4 ColorSaturationMidtones;
	float4 ColorContrastMidtones;
	float4 ColorGammaMidtones;
	float4 ColorGainMidtones;
	float4 ColorOffsetMidtones;

	float4 ColorSaturationHighlights;
	float4 ColorContrastHighlights;
	float4 ColorGammaHighlights;
	float4 ColorGainHighlights;
	float4 ColorOffsetHighlights;
	float HighlightsMin;
#endif
	float WhiteTemp;
	float Tint;
	float Inner;
	float Outer;

	float Falloff;
	float Invert;
	float Intensity;

	float FakeLight;
	uint   ExcludeStencil;

	float4 SceneTexture;
};

MaterialFloat3 ProcessColor(in FExternalExpressions ExternalExpressions)
{
	MaterialFloat3 ColorAP1 = mul((float3x3)WorkingColorSpace.ToAP1, ExternalExpressions.SceneTexture.xyz);
	MaterialFloat3 BasicColorGrade = ComposureColorGradePerRange(ColorAP1, ExternalExpressions.ColorSaturation, ExternalExpressions.ColorContrast, ExternalExpressions.ColorGamma, ExternalExpressions.ColorGain, ExternalExpressions.ColorOffset);
	MaterialFloat3 FinalColor = BasicColorGrade;
#if ADVANCED_CC	
	{
#define HIGHLIGHT_INTERPOLATION_THRESHOLD_WIDTH 1.0
		MaterialFloat3 Local3 = ComposureColorGradePerRange(BasicColorGrade, ExternalExpressions.ColorSaturationShadows, ExternalExpressions.ColorContrastShadows, ExternalExpressions.ColorGammaShadows, ExternalExpressions.ColorGainShadows, ExternalExpressions.ColorOffsetShadows);
		MaterialFloat BasicColorGrade_Mul = dot(BasicColorGrade, MaterialFloat3(0.30000001, 0.58999997, 0.11000000));
		// In the future we might want to add a threshold width to increase or reduce interpolation area between shadows and midtones.
		MaterialFloat Local5 = smoothstep(0.0 /*ExternalExpressions.ShadowMax - SHADOW_INTERPOLATION_THRESHOLD_WIDTH*/, ExternalExpressions.ShadowMax, BasicColorGrade_Mul);
		MaterialFloat Local6 = (1.0 - Local5);
		MaterialFloat3 Local7 = (Local3 * Local6);
		MaterialFloat3 Local8 = ComposureColorGradePerRange(BasicColorGrade, ExternalExpressions.ColorSaturationMidtones, ExternalExpressions.ColorContrastMidtones, ExternalExpressions.ColorGammaMidtones, ExternalExpressions.ColorGainMidtones, ExternalExpressions.ColorOffsetMidtones);
		// Interpolation threshold width specifies the upper bound for the soft interpolated area between midtones and highlights. 
		// In the future we want to add control that allows user to control this width.
		MaterialFloat Local9 = smoothstep(ExternalExpressions.HighlightsMin, ExternalExpressions.HighlightsMin + HIGHLIGHT_INTERPOLATION_THRESHOLD_WIDTH, BasicColorGrade_Mul);
		MaterialFloat Local10 = (Local6 + Local9);
		MaterialFloat Local11 = (1.0 - Local10);
		MaterialFloat3 Local12 = (Local8 * Local11);
		MaterialFloat3 Local13 = (Local7 + Local12);
		MaterialFloat3 Local14 = ComposureColorGradePerRange(BasicColorGrade, ExternalExpressions.ColorSaturationHighlights, ExternalExpressions.ColorContrastHighlights, ExternalExpressions.ColorGammaHighlights, ExternalExpressions.ColorGainHighlights, ExternalExpressions.ColorOffsetHighlights);
		MaterialFloat3 Local15 = (Local14 * Local9);
		FinalColor = (Local13 + Local15);
	}
#endif
	FinalColor = mul((float3x3)WorkingColorSpace.FromAP1, FinalColor.xyz);

	// White temperature calculation.
	MaterialFloat3 AfterTemperature = FinalColor;

#if TEMPERATURE_TYPE == TEMPERATURE_TYPE_LEGACY
	MaterialFloat Local4 = smoothstep(0.0, 13000.0, ExternalExpressions.WhiteTemp);
	MaterialFloat Local5 = (Local4 * 2.0 - 1.0);

	// MF Artistic temperature 
	{
		MaterialFloat Local7 = (Local5 * 3.0);
		MaterialFloat Local8 = clamp(Local7, 0.0, 5.0);
		MaterialFloat Local9 = (FinalColor.b * Local8);
		MaterialFloat Local10 = (Local9 * 0.36666000);
		MaterialFloat Local11 = (FinalColor.r + Local10);
		MaterialFloat Local12 = (1.0 - Local8);
		MaterialFloat Local13 = (Local12 * FinalColor.b);
		MaterialFloat Local14 = (Local5 * -3.0);
		MaterialFloat Local15 = clamp(Local14, 0.0, 5.0);
		MaterialFloat Local16 = (1.0 - Local15);
		MaterialFloat Local17 = (Local16 * FinalColor.r);
		MaterialFloat Local18 = (FinalColor.r * Local15);
		MaterialFloat Local19 = (Local18 * 2.72000003);
		MaterialFloat Local20 = (FinalColor.b + Local19);
		MaterialFloat Local21 = smoothstep(0.0001, 0.0, Local5);
		
		AfterTemperature = lerp(MaterialFloat3(Local11, FinalColor.g, Local13), MaterialFloat3(Local17, FinalColor.g, Local20), Local21); 
		AfterTemperature.g = AfterTemperature.g - ExternalExpressions.Tint; 
	}
#elif TEMPERATURE_TYPE == TEMPERATURE_TYPE_WHITE_BALANCE
	AfterTemperature = WhiteBalance(FinalColor, ExternalExpressions.WhiteTemp, ExternalExpressions.Tint, true /*bIsTemperatureWhiteBalance*/, (float3x3)WorkingColorSpace.ToXYZ, (float3x3)WorkingColorSpace.FromXYZ);
#elif TEMPERATURE_TYPE == TEMPERATURE_TYPE_COLOR_TEMP
	AfterTemperature = WhiteBalance(FinalColor, ExternalExpressions.WhiteTemp, ExternalExpressions.Tint, false /*bIsTemperatureWhiteBalance*/, (float3x3)WorkingColorSpace.ToXYZ, (float3x3)WorkingColorSpace.FromXYZ);
#endif
	return AfterTemperature;
}

#ifdef WINDOW_TYPE 
float GetShapeAlpha(in out FMaterialPixelParameters Parameters, in FExternalExpressions ExternalExpressions)
{
	// Pixel position in plane space.
	float3 RotatedTranslatedScaled = RotateTranslateScale(GetWorldPosition(Parameters), ExternalExpressions.Translate.xyz, ExternalExpressions.Rotate, ExternalExpressions.Scale);

	// Camera position in plane space.
	float3 CameraVectorPlaneSpace = RotateTranslateScale(DFHackToFloat(ResolvedView.WorldCameraOrigin).xyz, ExternalExpressions.Translate.xyz, ExternalExpressions.Rotate, ExternalExpressions.Scale);

	float3 PositionOnPlane;
	float DirectionSign;

	float IntersectionSide;
	{
		float3 PlaneNormal = float3(0., 0., -1.);
		float3 CamToPos = RotatedTranslatedScaled - CameraVectorPlaneSpace;
		float3 DirVector = normalize(CamToPos);
		DirectionSign = sign(dot(-PlaneNormal, DirVector));
		PlaneNormal *= DirectionSign;
		IntersectionSide = dot(-CameraVectorPlaneSpace, PlaneNormal);

		PositionOnPlane = CameraVectorPlaneSpace + IntersectionSide / dot(DirVector, PlaneNormal) * DirVector;
	}

	// Checks if Z is in front or behind the plane.
	float ClipZ = step(-RotatedTranslatedScaled.z * DirectionSign, 0.);
#if WINDOW_TYPE == WINDOW_TYPE_PLANE
	float2 Interpolator = smoothstep(ExternalExpressions.Outer, ExternalExpressions.Inner, abs(PositionOnPlane.xy)) * ClipZ * step(IntersectionSide, 0.);
	float ShapeAlpha = Interpolator.x * step(0., Interpolator.y) * Interpolator.g * step(0., Interpolator.x);
#elif WINDOW_TYPE == WINDOW_TYPE_CIRCLE
	float ShapeAlpha = smoothstep(ExternalExpressions.Outer, ExternalExpressions.Inner, length(PositionOnPlane)) * ClipZ * step(IntersectionSide, 0.);
#endif

#if CLIP_PIXELS_OUTSIDE_AABB
	// Clip pixel if it is outside the bounding box.
	{
		clip(ShapeAlpha  + ExternalExpressions.Invert - 0.00001);
	}
#endif

	return ShapeAlpha;
}
#else //#ifdef WINDOW_TYPE 

float GetShapeAlpha(in out FMaterialPixelParameters Parameters, in FExternalExpressions ExternalExpressions)
{
	float3 RotatedTranslatedScaled = RotateTranslateScale(GetWorldPosition(Parameters), ExternalExpressions.Translate.xyz, ExternalExpressions.Rotate, ExternalExpressions.Scale);

	float Shape = 0;

#if CLIP_PIXELS_OUTSIDE_AABB
	// Clip pixel if it is outside the bounding box.
	{
		float3 ComparisonShape_3D = max(RotatedTranslatedScaled, -RotatedTranslatedScaled);
		float ComparisonShape = max(max(ComparisonShape_3D.r, ComparisonShape_3D.g), ComparisonShape_3D.b);
		clip(1. - ComparisonShape * (1. - ExternalExpressions.Invert));
	}
#endif

#if SHAPE_TYPE == SHAPE_TYPE_BOX
	// Cube
	{
		float3 boxBoundsMax = smoothstep(ExternalExpressions.Inner, ExternalExpressions.Outer, RotatedTranslatedScaled);
		float3 boxBoundsMin = smoothstep(ExternalExpressions.Inner, ExternalExpressions.Outer, -RotatedTranslatedScaled);
		float3 result = max(boxBoundsMax, boxBoundsMin);
		Shape = max(max(result.r, result.g), result.b);
	}
#elif SHAPE_TYPE == SHAPE_TYPE_CYLINDER
	// cylinder
	{
		float Distance = length(RotatedTranslatedScaled.xy);
		float Local35 = smoothstep(ExternalExpressions.Inner, ExternalExpressions.Outer, Distance);
		float Local36 = smoothstep(ExternalExpressions.Inner, ExternalExpressions.Outer, RotatedTranslatedScaled.b);
		float Local39 = smoothstep(ExternalExpressions.Inner, ExternalExpressions.Outer, -RotatedTranslatedScaled.b);
		Shape = max(max(Local35, Local36), Local39);
	}
#elif SHAPE_TYPE == SHAPE_TYPE_CONE
	// cone
	{
		float local33 = length(RotatedTranslatedScaled.xy);
		float local36 = saturate(-RotatedTranslatedScaled.b);
		float local39 = smoothstep(ExternalExpressions.Inner * local36, ExternalExpressions.Outer * local36, local33);
		float local40 = smoothstep(ExternalExpressions.Inner, ExternalExpressions.Outer, RotatedTranslatedScaled.b);
		float local42 = smoothstep(ExternalExpressions.Inner, ExternalExpressions.Outer, -RotatedTranslatedScaled.b);
		Shape = max(max(local39, local40), local42);
	}
#elif SHAPE_TYPE == SHAPE_TYPE_SPHERE
	// Sphere
	{
		float Distance = length(RotatedTranslatedScaled);
		Shape = smoothstep(ExternalExpressions.Inner, ExternalExpressions.Outer, Distance);
	}
#endif
	float ShapeAlpha = (1.0 - Shape);
	return ShapeAlpha;
}
#endif //#ifdef WINDOW_TYPE 

MaterialFloat4 ProcessRegion(in out FMaterialPixelParameters Parameters, in FExternalExpressions ExternalExpressions)
{
#if STENCIL_ENABLED
	int CustomStencil = MergedStencilTexture.Load(int3(Parameters.SvPosition.xy, 0.)).r;

	{
		if (ExternalExpressions.ExcludeStencil == EXCLUDE_STENCIL)
		{
			if (CustomStencil > 0)
			{
				clip(-1.);
				return MaterialFloat4(ExternalExpressions.SceneTexture.rgb, 1.);
			}
		}
		else
		{
			if (CustomStencil == 0)
			{
				clip(-1.);
				return MaterialFloat4(ExternalExpressions.SceneTexture.rgb, 1.);
			}
		}
	}
#endif

	float ShapeAlpha = GetShapeAlpha(Parameters, ExternalExpressions);

	// Final check to reduce the performance based on shape.
	if (ShapeAlpha < 0.00001 && !ExternalExpressions.Invert)
	{
#if CCR_SHADER_DISPLAY_BOUNDING_RECT
		return float4(1, 0, 0, 0.4);
#else
		return float4(0, 0, 0, 0);
#endif
	}

	float3 AfterTemperature = ProcessColor(ExternalExpressions);

	/* Falloff */
	float ShapeWithFalloff = PositiveClampedPow(ShapeAlpha, ExternalExpressions.Falloff);
	float ShapeWithFalloffInverse = (1.0 - ShapeWithFalloff);

	float FinalShape = lerp(ShapeWithFalloff, ShapeWithFalloffInverse, ExternalExpressions.Invert);

	float FinalAlpha = lerp(0.0, ExternalExpressions.Intensity, FinalShape);

	MaterialFloat3 FinalColor = lerp(ExternalExpressions.SceneTexture.rgb, AfterTemperature, FinalAlpha);

	return MaterialFloat4(FinalColor, FinalAlpha);
}

void CalcPixelMaterialInputs(in out FMaterialPixelParameters Parameters, in out FPixelMaterialInputs PixelMaterialInputs)
{
    // The Normal is a special case as it might have its own expressions and also be used to calculate other inputs, so perform the assignment here
	PixelMaterialInputs.Normal = MaterialFloat3(0.0, 0.0, 1.0);

    // Note that here MaterialNormal can be in world space or tangent space
	half3 MaterialNormal = GetMaterialNormal(Parameters, PixelMaterialInputs);

	Parameters.WorldNormal = normalize(MaterialNormal);

	Parameters.ReflectionVector = ReflectionAboutCustomWorldNormal(Parameters, Parameters.WorldNormal, false);
	
	MaterialFloat4 SceneTexture = float4(View.OneOverPreExposure.xxx, 1) * SceneTextureLookup(GetDefaultSceneTextureUV(Parameters, PPI_PostProcessInput0), PPI_PostProcessInput0, false);
	
	MaterialFloat4 FinalColor = SceneTexture;
	
	{
		FExternalExpressions ExtExpressions;
		ExtExpressions.Rotate = RegionData.Rotate;
		ExtExpressions.Translate = RegionData.Translate;
		ExtExpressions.Scale = RegionData.Scale;

		ExtExpressions.ColorSaturation = ColorCorrectBase.ColorSaturation;
		ExtExpressions.ColorContrast = ColorCorrectBase.ColorContrast;
		ExtExpressions.ColorGamma = ColorCorrectBase.ColorGamma;
		ExtExpressions.ColorGain = ColorCorrectBase.ColorGain;
		ExtExpressions.ColorOffset = ColorCorrectBase.ColorOffset;
		ExtExpressions.WhiteTemp = RegionData.WhiteTemp;
		ExtExpressions.Tint = RegionData.Tint;
		ExtExpressions.Inner = RegionData.Inner;
		ExtExpressions.Outer = RegionData.Outer;
		ExtExpressions.Falloff = RegionData.Falloff;
		ExtExpressions.Invert = RegionData.Invert;
		ExtExpressions.Intensity = RegionData.Intensity;
		ExtExpressions.FakeLight = RegionData.FakeLight;
		ExtExpressions.ExcludeStencil = RegionData.ExcludeStencil;
		ExtExpressions.SceneTexture = SceneTexture.rgba;
#if ADVANCED_CC	
		{
			ExtExpressions.ColorSaturationShadows = ColorCorrectShadows.ColorSaturation;
			ExtExpressions.ColorContrastShadows = ColorCorrectShadows.ColorContrast;
			ExtExpressions.ColorGammaShadows = ColorCorrectShadows.ColorGamma;
			ExtExpressions.ColorGainShadows = ColorCorrectShadows.ColorGain;
			ExtExpressions.ColorOffsetShadows = ColorCorrectShadows.ColorOffset;
			ExtExpressions.ShadowMax = ColorCorrectShadows.ShadowMax;
			
			ExtExpressions.ColorSaturationMidtones = ColorCorrectMidtones.ColorSaturation;
			ExtExpressions.ColorContrastMidtones = ColorCorrectMidtones.ColorContrast;
			ExtExpressions.ColorGammaMidtones = ColorCorrectMidtones.ColorGamma;
			ExtExpressions.ColorGainMidtones = ColorCorrectMidtones.ColorGain;
			ExtExpressions.ColorOffsetMidtones = ColorCorrectMidtones.ColorOffset;
			
			ExtExpressions.ColorSaturationHighlights = ColorCorrectHighlights.ColorSaturation;
			ExtExpressions.ColorContrastHighlights = ColorCorrectHighlights.ColorContrast;
			ExtExpressions.ColorGammaHighlights = ColorCorrectHighlights.ColorGamma;
			ExtExpressions.ColorGainHighlights = ColorCorrectHighlights.ColorGain;
			ExtExpressions.ColorOffsetHighlights = ColorCorrectHighlights.ColorOffset;
			ExtExpressions.HighlightsMin = ColorCorrectHighlights.HighlightsMin;
		}
#endif

		MaterialFloat4 RegionColor = ProcessRegion(Parameters, ExtExpressions);
		FinalColor = lerp(FinalColor.rgba, RegionColor.rgba, RegionColor.a);
	}
		
	PixelMaterialInputs.EmissiveColor = FinalColor.rgb;
#if CLIP_PIXELS_OUTSIDE_AABB
	PixelMaterialInputs.Opacity = FinalColor.a;
#else
	PixelMaterialInputs.Opacity = 1.0;
#endif

	PixelMaterialInputs.OpacityMask = 1.0;
	PixelMaterialInputs.BaseColor = MaterialFloat3(0.0, 0.0, 0.0);
	PixelMaterialInputs.Metallic = 0.0;
	PixelMaterialInputs.Specular = 0.50000000;
	PixelMaterialInputs.Roughness = 0.50000000;
	PixelMaterialInputs.Anisotropy = 0.0;
	PixelMaterialInputs.Tangent = MaterialFloat3(1.0, 0.0, 0.0);
	PixelMaterialInputs.Subsurface = 0;
	PixelMaterialInputs.AmbientOcclusion = 1.0;
	PixelMaterialInputs.Refraction = 0;
	PixelMaterialInputs.PixelDepthOffset = 0.0;
	PixelMaterialInputs.ShadingModel = 0;
	Parameters.WorldTangent = 0;
}

struct FPostProcessMaterialVSToPS
{
	float4 Position : SV_POSITION;
};

#if (FEATURE_LEVEL > FEATURE_LEVEL_ES3_1)
void MainVS(
	in float4 InPosition : ATTRIBUTE0,
	out FPostProcessMaterialVSToPS Output
	)
{
	Output = (FPostProcessMaterialVSToPS)0;
	DrawRectangle(InPosition, Output.Position);

}
void BuildParameters(in FPostProcessMaterialVSToPS Input, inout FMaterialPixelParameters Parameters)
{
	Parameters = (FMaterialPixelParameters)0;

	ResolvedView = ResolveView();

	float4 SvPosition = Input.Position;
	float2 ViewportUV = (SvPosition.xy - PostProcessOutput_ViewportMin.xy) * PostProcessOutput_ViewportSizeInverse.xy;

	SvPosition.z = LookupDeviceZ(ViewportUVToBufferUV(ViewportUV));
	SvPosition.z = max(SvPosition.z, 1e-18);

	float4 ScreenPosition = SvPositionToScreenPosition(SvPosition);
	float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(SvPosition);

	// Remove the pre view translation
	Parameters.WorldPosition_CamRelative = TranslatedWorldPosition.xyz;
	Parameters.AbsoluteWorldPosition = TranslatedWorldPosition.xyz - DFHackToFloat(ResolvedView.PreViewTranslation).xyz;

	Parameters.SvPosition = SvPosition;
	Parameters.ScreenPosition = ScreenPosition;

	Parameters.CameraVector = normalize(-Parameters.WorldPosition_CamRelative.xyz);
}

void MainPS(
	in FPostProcessMaterialVSToPS Input,
	out float4 OutColor : SV_Target0
	)
{
	FMaterialPixelParameters Parameters;
	FPixelMaterialInputs PixelMaterialInputs;

	BuildParameters(Input, Parameters);

    // Now that we have all the pixel-related parameters setup, calculate the Material Input/Attributes and Normal
	CalcPixelMaterialInputs(Parameters, PixelMaterialInputs);
	
	const float Alpha = saturate(PixelMaterialInputs.Opacity);

	OutColor = float4(max(PixelMaterialInputs.EmissiveColor, 0.), Alpha);
	OutColor.xyz *= View.PreExposure;
} 

#endif 
