// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/DeferredShadingCommon.ush"
#include "/Engine/Private/SceneTexturesCommon.ush"
#include "/Engine/Private/ScreenPass.ush"
#include "/Engine/Private/SceneTextureParameters.ush"
#include "/Engine/Private/SceneData.ush"
#include "/ColorCorrectRegionsShaders/Private/ColorCorrectRegionsCommon.usf"


StructuredBuffer<uint> StencilIds;
uint StencilIdCount;

uint MainPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0
) : SV_Target0
{
#if SHADING_PATH_DEFERRED
	float4 SvPosition = UVAndScreenPos;
	const float Float_Eps = 0.00000000001;

	float CustomDepth = CalcSceneCustomDepth(UVAndScreenPos.xy);
	float SceneDepth = CalcSceneDepth(UVAndScreenPos.xy);

	if (abs(SceneDepth - CustomDepth) <= Float_Eps)
	{
		int CustomStencil = SceneTextureLookup(UVAndScreenPos.xy, PPI_CustomStencil, false).r;
		for (uint Index = 0; Index < StencilIdCount; Index++)
		{
			if (CustomStencil == StencilIds[Index])
			{
				return CustomStencil;
			}
		}
	}
#endif
	return 0;
}
