// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class ColorCorrectRegions : ModuleRules
{
	public ColorCorrectRegions(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"DisplayClusterLightCardExtender"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"RHI",
				"Renderer",
				"Projects",
				"RenderCore",
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
			PrivateDependencyModuleNames.Add("EditorWidgets");
			PrivateDependencyModuleNames.Add("Slate");
			PrivateDependencyModuleNames.Add("SlateCore");

			// Multi-user support: synchronise per actor CC stencil assignments via Concert
			// custom events so passive clients don't rely on tick-based polling.
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ConcertSyncClient",
				}
			);
		}
	}
}
