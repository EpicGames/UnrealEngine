// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorCorrectRegion.h"
#include "ColorCorrectRegionsModule.h"
#include "ColorCorrectRegionsSubsystem.h"
#include "ColorCorrectWindow.h"
#include "Components/BillboardComponent.h"
#include "Components/MeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "CoreMinimal.h"
#include "Engine/CollisionProfile.h"
#include "Engine/GameEngine.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Materials/Material.h"
#include "UObject/ConstructorHelpers.h"

#if WITH_EDITOR
#include "IDisplayClusterLightCardExtenderModule.h"
#include "Misc/TransactionObjectEvent.h"
#endif

ENUM_RANGE_BY_COUNT(EColorCorrectRegionsType, EColorCorrectRegionsType::MAX)


AColorCorrectRegion::AColorCorrectRegion(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Type(EColorCorrectRegionsType::Sphere)
	, Priority(0)
	, Intensity(1.0)
	, Inner(0.5)
	, Outer(1.0)
	, Falloff(1.0)
	, Invert(false)
	, TemperatureType(EColorCorrectRegionTemperatureType::ColorTemperature)
	, Temperature(6500)
	, Tint(0)
	, Enabled(true)
	, bEnablePerActorCC(false)
	, PerActorColorCorrection(EColorCorrectRegionStencilType::IncludeStencil)
{
	PrimaryActorTick.bCanEverTick = true;

	// Add a scene component as our root
	RootComponent = ObjectInitializer.CreateDefaultSubobject<USceneComponent>(this, TEXT("Root"));
	RootComponent->SetMobility(EComponentMobility::Movable);

	IdentityComponent = CreateDefaultSubobject<UColorCorrectionInvisibleComponent>("IdentityComponent");
	IdentityComponent->SetupAttachment(RootComponent);
	IdentityComponent->CastShadow = false;
	IdentityComponent->SetHiddenInGame(false);

#if WITH_EDITOR
	if (!IsTemplate())
	{
		IDisplayClusterLightCardExtenderModule& LightCardExtenderModule = IDisplayClusterLightCardExtenderModule::Get();
		LightCardExtenderModule.GetOnSequencerTimeChanged().AddUObject(this, &AColorCorrectRegion::OnSequencerTimeChanged);
	}
#endif
}

bool AColorCorrectRegion::ShouldTickIfViewportsOnly() const
{
	return true;
}


#if WITH_EDITOR
void AColorCorrectRegion::OnSequencerTimeChanged(TWeakPtr<ISequencer> InSequencer)
{
	bNotifyOnParamSetter = false;
	UpdatePositionalParamsFromTransform();
	bNotifyOnParamSetter = true;
}
#endif

void AColorCorrectRegion::HandleAffectedActorsPropertyChange(uint32 ActorListChangeType)
{
	TWeakObjectPtr<UColorCorrectRegionsSubsystem> ColorCorrectRegionsSubsystem;
	if (const UWorld* World = GetWorld())
	{
		ColorCorrectRegionsSubsystem = World->GetSubsystem<UColorCorrectRegionsSubsystem>();
	}

	if (ActorListChangeType == EPropertyChangeType::ArrayClear
		|| ActorListChangeType == EPropertyChangeType::ArrayRemove
		|| ActorListChangeType == EPropertyChangeType::ValueSet
		|| ActorListChangeType == EPropertyChangeType::Unspecified)
	{
		if (ColorCorrectRegionsSubsystem.IsValid())
		{
			ColorCorrectRegionsSubsystem->ClearStencilIdsToPerActorCC(this);
		}
	}

	if (ActorListChangeType == EPropertyChangeType::ArrayAdd
		|| ActorListChangeType == EPropertyChangeType::ValueSet
		|| ActorListChangeType == EPropertyChangeType::Unspecified)
	{
		// In case user assigns Color Correct Region or Window, we should remove it as it is invalid operation.
		{
			TArray<TSoftObjectPtr<AActor>> ActorsToRemove;
			for (const TSoftObjectPtr<AActor>& StencilActor : AffectedActors)
			{
				if (AColorCorrectRegion* CCRCast = Cast<AColorCorrectRegion>(StencilActor.Get()))
				{
					ActorsToRemove.Add(StencilActor);
				}
			}
			if (ActorsToRemove.Num() > 0)
			{
				UE_LOGF(ColorCorrectRegions, Warning, "Color Correct Region or Window assignment to Per Actor CC is not supported.");
			}
			for (const TSoftObjectPtr<AActor>& StencilActor : ActorsToRemove)
			{
				AffectedActors.Remove(StencilActor);
				AffectedActors.FindOrAdd(TSoftObjectPtr<AActor>());
			}
		}

		if (ColorCorrectRegionsSubsystem.IsValid())
		{
			ColorCorrectRegionsSubsystem->AssignStencilIdsToPerActorCC(this);
		}
	}

	// Notify connected multi-user clients of the updated AffectedActors list via a Concert custom event
	if (ColorCorrectRegionsSubsystem.IsValid())
	{
		ColorCorrectRegionsSubsystem->NotifyPerActorCCChanged(this);
	}
}

#if WITH_METADATA
void AColorCorrectionRegion::CreateIcon()
{
	// Create billboard component
	if (GIsEditor && !IsRunningCommandlet())
	{
		// Structure to hold one-time initialization

		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTextureObject;
			FName ID_ColorCorrectRegion;
			FText NAME_ColorCorrectRegion;

			FConstructorStatics()
				: SpriteTextureObject(TEXT("/ColorCorrectRegions/Icons/S_ColorCorrectRegionIcon"))
				, ID_ColorCorrectRegion(TEXT("Color Correct Region"))
				, NAME_ColorCorrectRegion(NSLOCTEXT("SpriteCategory", "ColorCorrectRegion", "Color Correct Region"))
			{
			}
		};

		static FConstructorStatics ConstructorStatics;

		SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Color Correct Region Icon"));

		if (SpriteComponent)
		{
			SpriteComponent->Sprite = ConstructorStatics.SpriteTextureObject.Get();
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_ColorCorrectRegion;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_ColorCorrectRegion;
			SpriteComponent->SetIsVisualizationComponent(true);
			SpriteComponent->SetRelativeLocationAndRotation(FVector::ZeroVector, FRotator::ZeroRotator);
			SpriteComponent->SetMobility(EComponentMobility::Movable);
			SpriteComponent->bHiddenInGame = true;
			SpriteComponent->bIsScreenSizeScaled = true;

			SpriteComponent->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepRelativeTransform);
		}
	}

}
#endif 

AColorCorrectRegion::~AColorCorrectRegion()
{
#if WITH_EDITOR
	if (!IsTemplate())
	{
		IDisplayClusterLightCardExtenderModule& LightCardExtenderModule = IDisplayClusterLightCardExtenderModule::Get();
		LightCardExtenderModule.GetOnSequencerTimeChanged().RemoveAll(this);
	}
#endif
}

#if WITH_EDITOR
void AColorCorrectRegion::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	// Use MemberProperty (the class-level field) rather than the leaf property name.
	// When resetting an array element to default, the leaf property is the inner element type,
	// not "AffectedActors" itself, so GetPropertyName() would miss it.
	const FName MemberPropertyName = PropertyChangedEvent.MemberProperty
		? PropertyChangedEvent.MemberProperty->GetFName()
		: PropertyName;

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, AffectedActors))
	{
		HandleAffectedActorsPropertyChange(PropertyChangedEvent.ChangeType);
	}

	// Stage actor properties
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(PropertyChangedEvent.MemberProperty);
		const bool bIsOrientation = StructProperty ? StructProperty->Struct == FDisplayClusterPositionalParams::StaticStruct() : false;
	
		if (bIsOrientation)
		{
			UpdateStageActorTransform();
			// Updates MU in real-time. Skip our method as the positional coordinates are already correct.
			AActor::PostEditMove(PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive);
		}
		else if (
			PropertyName == USceneComponent::GetRelativeLocationPropertyName() ||
			PropertyName == USceneComponent::GetRelativeRotationPropertyName() ||
			PropertyName == USceneComponent::GetRelativeScale3DPropertyName())
		{
			bNotifyOnParamSetter = false;
			UpdatePositionalParamsFromTransform();
			bNotifyOnParamSetter = true;
		}
	}

	// Call after stage actor transform is updated, so any observers will have both the correct actor transform and
	// positional properties.
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void AColorCorrectRegion::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	bNotifyOnParamSetter = false;
	UpdatePositionalParamsFromTransform();
	bNotifyOnParamSetter = true;
}

void AColorCorrectRegion::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);
	FixMeshComponentReferences();

	// PerAffectedActorStencilData is non-UPROPERTY so the transaction system never restores it.
	// Clear it on undo/redo so it stays consistent with the transaction-restored AffectedActors.
	// The next stencil assignment will repopulate it.
	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		PerAffectedActorStencilData.Empty();
	}
}
#endif //WITH_EDITOR

#define NOTIFY_PARAM_SETTER()\
	if (bNotifyOnParamSetter)\
	{\
		UpdateStageActorTransform();\
	}\

void AColorCorrectRegion::SetLongitude(double InValue)
{
	PositionalParams.Longitude = InValue;
	NOTIFY_PARAM_SETTER()
}

double AColorCorrectRegion::GetLongitude() const
{
	return PositionalParams.Longitude;
}

void AColorCorrectRegion::SetLatitude(double InValue)
{
	PositionalParams.Latitude = InValue;
	NOTIFY_PARAM_SETTER()
}

double AColorCorrectRegion::GetLatitude() const
{
	return PositionalParams.Latitude;
}

void AColorCorrectRegion::SetDistanceFromCenter(double InValue)
{
	PositionalParams.DistanceFromCenter = InValue;
	NOTIFY_PARAM_SETTER()
}

double AColorCorrectRegion::GetDistanceFromCenter() const
{
	return PositionalParams.DistanceFromCenter;
}

void AColorCorrectRegion::SetSpin(double InValue)
{
	PositionalParams.Spin = InValue;
	NOTIFY_PARAM_SETTER()
}

double AColorCorrectRegion::GetSpin() const
{
	return PositionalParams.Spin;
}

void AColorCorrectRegion::SetPitch(double InValue)
{
	PositionalParams.Pitch = InValue;
	NOTIFY_PARAM_SETTER()
}

double AColorCorrectRegion::GetPitch() const
{
	return PositionalParams.Pitch;
}

void AColorCorrectRegion::SetYaw(double InValue)
{
	PositionalParams.Yaw = InValue;
	NOTIFY_PARAM_SETTER()
}

double AColorCorrectRegion::GetYaw() const
{
	return PositionalParams.Yaw;
}

void AColorCorrectRegion::SetRadialOffset(double InValue)
{
	PositionalParams.RadialOffset = InValue;
	NOTIFY_PARAM_SETTER()
}

double AColorCorrectRegion::GetRadialOffset() const
{
	return PositionalParams.RadialOffset;
}

void AColorCorrectRegion::SetScale(const FVector2D& InScale)
{
	PositionalParams.Scale = InScale;
	NOTIFY_PARAM_SETTER()
}

FVector2D AColorCorrectRegion::GetScale() const
{
	return PositionalParams.Scale;
}

void AColorCorrectRegion::SetOrigin(const FTransform& InOrigin)
{
	Origin = InOrigin;
}

FTransform AColorCorrectRegion::GetOrigin() const
{
	return Origin;
}

void AColorCorrectRegion::SetPositionalParams(const FDisplayClusterPositionalParams& InParams)
{
	PositionalParams = InParams;
	NOTIFY_PARAM_SETTER()
}

FDisplayClusterPositionalParams AColorCorrectRegion::GetPositionalParams() const
{
	return PositionalParams;
}

void AColorCorrectRegion::GetPositionalProperties(FPositionalPropertyArray& OutPropertyPairs) const
{
	void* Container = (void*)(&PositionalParams);

	const TSet<FName>& PropertyNames = GetPositionalPropertyNames();
	OutPropertyPairs.Reserve(PropertyNames.Num());

	for (const FName& PropertyName : PropertyNames)
	{
		if (FProperty* Property = FindFProperty<FProperty>(FDisplayClusterPositionalParams::StaticStruct(), PropertyName))
		{
			OutPropertyPairs.Emplace(Container, Property);
		}
	}

	if (FStructProperty* ParamsProperty = FindFProperty<FStructProperty>(GetClass(), GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, PositionalParams)))
	{
		OutPropertyPairs.Emplace((void*)this, ParamsProperty);
	}
}

FName AColorCorrectRegion::GetPositionalPropertiesMemberName() const
{
	return GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, PositionalParams);
}

AColorCorrectionRegion::AColorCorrectionRegion(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

#if WITH_METADATA
		CreateIcon();
#endif

	UMaterial* Material = LoadObject<UMaterial>(NULL, TEXT("/ColorCorrectRegions/Materials/M_ColorCorrectRegionTransparentPreview.M_ColorCorrectRegionTransparentPreview"), NULL, LOAD_None, NULL);
	const TArray<UStaticMesh*> StaticMeshes =
	{
		Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, TEXT("/Engine/BasicShapes/Sphere"))),
		Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, TEXT("/Engine/BasicShapes/Cube"))),
		Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, TEXT("/Engine/BasicShapes/Cylinder"))),
		Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, TEXT("/Engine/BasicShapes/Cone")))
	};

	for (EColorCorrectRegionsType CCRType : TEnumRange<EColorCorrectRegionsType>())
	{
		UStaticMeshComponent* MeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(*UEnum::GetValueAsString(CCRType));
		MeshComponents.Add(MeshComponent);
		MeshComponent->SetupAttachment(RootComponent);
		MeshComponent->SetStaticMesh(StaticMeshes[static_cast<uint8>(CCRType)]);
		MeshComponent->SetMaterial(0, Material);
		MeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		MeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		MeshComponent->CastShadow = false;
		MeshComponent->SetHiddenInGame(true);
	}
	ChangeShapeVisibilityForActorType();
}

void AColorCorrectionRegion::ChangeShapeVisibilityForActorType()
{
	ChangeShapeVisibilityForActorTypeInternal<EColorCorrectRegionsType>(Type);
}

#if WITH_EDITOR
void AColorCorrectionRegion::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, Type) || PropertyChangedEvent.Property == nullptr)
	{
		ChangeShapeVisibilityForActorType();
	}
}

FName AColorCorrectionRegion::GetCustomIconName() const
{
	return TEXT("CCR.OutlinerThumbnail");
}

void AColorCorrectionRegion::FixMeshComponentReferences()
{
	FixMeshComponentReferencesInternal<EColorCorrectRegionsType>(Type);
}
#endif

#undef NOTIFY_PARAM_SETTER
