// Copyright Epic Games, Inc. All Rights Reserved.
#include "ColorCorrectRegionsSubsystem.h"
#include "ColorCorrectRegionsModule.h"
#include "ColorCorrectRegionsSceneViewExtension.h"
#include "ColorCorrectRegionsStencilManager.h"
#include "ColorCorrectWindow.h"

#include "Engine/World.h"
#include "Kismet/GameplayStatics.h"
#include "EngineUtils.h"
#include "SceneViewExtension.h"


#if WITH_EDITOR
#include "Editor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ScopedTransaction.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#define LOCTEXT_NAMESPACE "CCR"

static TAutoConsoleVariable<int32> CVarCCRPriorityIncrement(
	TEXT("r.CCR.PriorityIncrementAmount"),
	1,
	TEXT("Affects the priority increment of a newly created Color Correct Region."));

namespace
{
	bool IsRegionValid(AColorCorrectRegion* InRegion, UWorld* CurrentWorld)
	{
		// There some cases in which actor can belong to a different world or the world without subsystem.
		// Example: when editing a blueprint deriving from AVPCRegion.
		// We also check if the actor is being dragged from the content browser.
#if WITH_EDITOR
		return InRegion && !InRegion->bIsEditorPreviewActor && InRegion->GetWorld() == CurrentWorld;
#else
		return InRegion && InRegion->GetWorld() == CurrentWorld;
#endif
	}

	void AssignNewPriorityIfNeeded(AColorCorrectRegion* InRegion, const TArray<TWeakObjectPtr<AColorCorrectRegion>>& RegionsPriorityBased)
	{
		int32 HighestPriority = 0;
		bool bAssignNewPriority = InRegion->Priority == 0;
		for (const TWeakObjectPtr<AColorCorrectRegion> Region : RegionsPriorityBased)
		{
			if (InRegion->Priority == Region->Priority)
			{
				bAssignNewPriority = true;
			}
			HighestPriority = HighestPriority < Region->Priority ? Region->Priority : HighestPriority;
		}
		if (bAssignNewPriority)
		{
			InRegion->Priority = HighestPriority + (HighestPriority == 0 ? 1 : FMath::Max(CVarCCRPriorityIncrement.GetValueOnAnyThread(), 1));
#if WITH_EDITOR
			const FScopedTransaction Transaction(LOCTEXT("NewPriorityAssigned", "New Priority Assigned to CC Actor."));
			InRegion->Modify();
#endif
		}
	}

	FColorCorrectRenderProxyPtr CreateRenderStateForCCActor(TWeakObjectPtr<AColorCorrectRegion> InActorWeakPtr, const bool bInSupportsStencil)
	{
		TStrongObjectPtr<AColorCorrectRegion> InActor = InActorWeakPtr.Pin();
		if (!InActor.IsValid() || !IsValid(InActor->GetWorld()))
		{
			return nullptr;
		}

		FColorCorrectRenderProxyPtr TempCCRStateRenderThread = MakeShared<FColorCorrectRenderProxy>();

		TempCCRStateRenderThread->bIsActiveThisFrame = InActor->Enabled
#if WITH_EDITOR
			&& !InActor->IsHiddenEd()
#endif 
			&& !(InActor->GetWorld()->HasBegunPlay() && InActor->IsHidden());

		if (!TempCCRStateRenderThread->bIsActiveThisFrame)
		{
			return nullptr;
		}

		if (AColorCorrectionWindow* CCWindow = Cast<AColorCorrectionWindow>(InActor.Get()))
		{
			TempCCRStateRenderThread->WindowType = CCWindow->WindowType;
			TempCCRStateRenderThread->ProxyType = FColorCorrectRenderProxy::DistanceBased;
		}
		else
		{
			TempCCRStateRenderThread->Type = InActor->Type;
			TempCCRStateRenderThread->ProxyType = FColorCorrectRenderProxy::PriorityBased;
		}

		TempCCRStateRenderThread->World = InActor->GetWorld();
		TempCCRStateRenderThread->Priority = InActor->Priority;
		TempCCRStateRenderThread->Intensity = InActor->Intensity;

		// Inner could be larger than outer, in which case we need to make sure these are swapped.
		TempCCRStateRenderThread->Inner = FMath::Min<float>(InActor->Outer, InActor->Inner);
		TempCCRStateRenderThread->Outer = FMath::Max<float>(InActor->Outer, InActor->Inner);

		if (TempCCRStateRenderThread->Inner == TempCCRStateRenderThread->Outer)
		{
			TempCCRStateRenderThread->Inner -= 0.0001f;
		}

		TempCCRStateRenderThread->Falloff = InActor->Falloff;
		TempCCRStateRenderThread->Invert = InActor->Invert;
		TempCCRStateRenderThread->TemperatureType = InActor->TemperatureType;
		TempCCRStateRenderThread->Temperature = InActor->Temperature;
		TempCCRStateRenderThread->Tint = InActor->Tint;
		TempCCRStateRenderThread->ColorGradingSettings = InActor->ColorGradingSettings;
		TempCCRStateRenderThread->bEnablePerActorCC = bInSupportsStencil && InActor->bEnablePerActorCC;
		TempCCRStateRenderThread->PerActorColorCorrection = InActor->PerActorColorCorrection;

		InActor->GetActorBounds(false, TempCCRStateRenderThread->BoxOrigin, TempCCRStateRenderThread->BoxExtent);

		TempCCRStateRenderThread->ActorLocation = (FVector3f)InActor->GetActorLocation();
		TempCCRStateRenderThread->ActorRotation = (FVector3f)InActor->GetActorRotation().Euler();
		TempCCRStateRenderThread->ActorScale = (FVector3f)InActor->GetActorScale();

		// Transfer Stencil Ids.
		{

			for (const TSoftObjectPtr<AActor>& StencilActor : InActor->AffectedActors)
			{
				if (!StencilActor.IsValid())
				{
					continue;
				}
				TArray<UPrimitiveComponent*> PrimitiveComponents;
				StencilActor->GetComponents<UPrimitiveComponent>(PrimitiveComponents);
				for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
				{
					if (PrimitiveComponent->bRenderCustomDepth)
					{
						TempCCRStateRenderThread->StencilIds.Add(static_cast<uint32>(PrimitiveComponent->CustomDepthStencilValue));
					}
				}
			}
		}

		// Store component id to be used on render thread.
		if (!(TempCCRStateRenderThread->FirstPrimitiveId == InActor->IdentityComponent->GetPrimitiveSceneId()))
		{
			TempCCRStateRenderThread->FirstPrimitiveId = InActor->IdentityComponent->GetPrimitiveSceneId();
		}
		return TempCCRStateRenderThread;
	}
}

void UColorCorrectRegionsSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
#if WITH_EDITOR
	if (GetWorld()->WorldType == EWorldType::Editor)
	{
		GEngine->OnLevelActorAdded().AddUObject(this, &UColorCorrectRegionsSubsystem::OnActorSpawned);
		GEngine->OnLevelActorDeleted().AddUObject(this, &UColorCorrectRegionsSubsystem::OnActorDeleted, true);

		FEditorDelegates::OnDuplicateActorsBegin.AddUObject(this, &UColorCorrectRegionsSubsystem::OnDuplicateActorsBegin);
		FEditorDelegates::OnDuplicateActorsEnd.AddUObject(this, &UColorCorrectRegionsSubsystem::OnDuplicateActorsEnd);

		FEditorDelegates::OnEditPasteActorsBegin.AddUObject(this, &UColorCorrectRegionsSubsystem::OnDuplicateActorsBegin);
		FEditorDelegates::OnEditPasteActorsEnd.AddUObject(this, &UColorCorrectRegionsSubsystem::OnDuplicateActorsEnd);
	}
#endif

	// Initializing Scene view extension responsible for rendering regions.
	PostProcessSceneViewExtension = FSceneViewExtensions::NewExtension<FColorCorrectRegionsSceneViewExtension>(GetWorld(), this);

#if WITH_EDITOR
	if (GetWorld()->WorldType == EWorldType::Editor)
	{
		MultiUserManager = MakeUnique<FCCRMultiUserManager>(this);
	}
#endif

	Super::Initialize(Collection);
}

void UColorCorrectRegionsSubsystem::Deinitialize()
{
#if WITH_EDITOR
	if (GetWorld()->WorldType == EWorldType::Editor)
	{
		MultiUserManager.Reset();

		GEngine->OnLevelActorAdded().RemoveAll(this);
		GEngine->OnLevelActorDeleted().RemoveAll(this);

		FEditorDelegates::OnDuplicateActorsBegin.RemoveAll(this);
		FEditorDelegates::OnDuplicateActorsEnd.RemoveAll(this);

		FEditorDelegates::OnEditPasteActorsBegin.RemoveAll(this);
		FEditorDelegates::OnEditPasteActorsEnd.RemoveAll(this);
	}
#endif

	// Prevent this SVE from being gathered, in case it is kept alive by a strong reference somewhere else.
	{
		PostProcessSceneViewExtension->IsActiveThisFrameFunctions.Empty();

		FSceneViewExtensionIsActiveFunctor IsActiveFunctor;

		IsActiveFunctor.IsActiveFunction = [](const ISceneViewExtension* SceneViewExtension, const FSceneViewExtensionContext& Context)
		{
			return TOptional<bool>(false);
		};

		PostProcessSceneViewExtension->IsActiveThisFrameFunctions.Add(IsActiveFunctor);
	}

	ENQUEUE_RENDER_COMMAND(ReleaseSVE)([this](FRHICommandListImmediate& RHICmdList)
	{
		// Prevent this SVE from being gathered, in case it is kept alive by a strong reference somewhere else.
		{
			PostProcessSceneViewExtension->IsActiveThisFrameFunctions.Empty();

			FSceneViewExtensionIsActiveFunctor IsActiveFunctor;

			IsActiveFunctor.IsActiveFunction = [](const ISceneViewExtension* SceneViewExtension, const FSceneViewExtensionContext& Context)
			{
				return TOptional<bool>(false);
			};

			PostProcessSceneViewExtension->IsActiveThisFrameFunctions.Add(IsActiveFunctor);
		}

		PostProcessSceneViewExtension->Invalidate();
		PostProcessSceneViewExtension.Reset();
		PostProcessSceneViewExtension = nullptr;
	});

	// Finish all rendering commands before cleaning up actors.
	FlushRenderingCommands();

	RegionsPriorityBased.Reset();
	RegionsDistanceBased.Reset();
	Super::Deinitialize();
}

void UColorCorrectRegionsSubsystem::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);
	RefreshRegions();

	// Check to make sure that no ids have been changed externally.
	{
		TimeSinceLastValidityCheck += DeltaTime;
		const float WaitTimeInSecs = 1.0;
		if (TimeSinceLastValidityCheck >= WaitTimeInSecs)
		{
			auto CheckRegion = [this](const TWeakObjectPtr<AColorCorrectRegion>& Region)
			{
				if (!Region.IsValid())
				{
					return;
				}
				CheckAssignedActorsValidity(Region.Get());
			};

			for (const TWeakObjectPtr<AColorCorrectRegion>& Region : RegionsPriorityBased)
			{
				CheckRegion(Region);
			}
			for (const TWeakObjectPtr<AColorCorrectRegion>& Region : RegionsDistanceBased)
			{
				CheckRegion(Region);
			}

			TimeSinceLastValidityCheck = 0;
		}
	}
	
}

void UColorCorrectRegionsSubsystem::TransferStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CCR.TransferStates);

	TArray<FColorCorrectRenderProxyPtr> TempProxiesPriority;
	TArray<FColorCorrectRenderProxyPtr> TempProxiesDistance;

	// Custom Depth required to be set to Enabled with stencil for Per Actor CC feature to work.
	static bool bNotifiedAboutCustomDepth = false;
	static const auto CVarCustomDepth = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.CustomDepth"));
	constexpr int32 EnabledWithStencil = 3;
	const bool bSupportsStencil = CVarCustomDepth->GetValueOnAnyThread() == EnabledWithStencil;

	for (TWeakObjectPtr<AColorCorrectRegion> Region : RegionsPriorityBased)
	{
		if (FColorCorrectRenderProxyPtr ProxyPtr = CreateRenderStateForCCActor(Region, bSupportsStencil))
		{
			TempProxiesPriority.Add(ProxyPtr);
		}
	}

	for (TWeakObjectPtr<AColorCorrectRegion> Region : RegionsDistanceBased)
	{
		if (FColorCorrectRenderProxyPtr ProxyPtr = CreateRenderStateForCCActor(Region, bSupportsStencil))
		{
			TempProxiesDistance.Add(ProxyPtr);
		}
	}

	// Sort priority based proxies on game thread.
	TempProxiesPriority.Sort([](const FColorCorrectRenderProxyPtr& A, const FColorCorrectRenderProxyPtr& B) {
		// Regions with the same priority could potentially cause flickering on overlap
		return A->Priority < B->Priority;
	});

	const int32 TransferCount = TempProxiesPriority.Num() + TempProxiesDistance.Num();
	if (TransferCount > 0)
	{
		if (!bNotifiedAboutCustomDepth && !bSupportsStencil)
		{
			FString InvalidCustomDepthSettingString = "Per Actor Color Correction requires Custom Depth Mode to be set to \"Enabled With Stencil\"";
			UE_LOGF(ColorCorrectRegions, Error, "%ls", *InvalidCustomDepthSettingString);

#if WITH_EDITOR
			FNotificationInfo Info(FText::FromString(InvalidCustomDepthSettingString));
			Info.ExpireDuration = 5.0f;
			Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));

			FSlateNotificationManager::Get().AddNotification(Info);
#endif

			bNotifiedAboutCustomDepth = true;
		}
	}

	ENQUEUE_RENDER_COMMAND(CopyCCProxies)([this, TempProxiesPriority = MoveTemp(TempProxiesPriority), TempProxiesDistance = MoveTemp(TempProxiesDistance)](FRHICommandListImmediate& RHICmdList)
	{
		ProxiesPriorityBased = TempProxiesPriority;
		ProxiesDistanceBased = TempProxiesDistance;
	}
	);
}


void UColorCorrectRegionsSubsystem::OnActorSpawned(AActor* InActor)
{
	if (bDuplicationStarted)
	{
		DuplicatedActors.Add(InActor);
	}
}

void UColorCorrectRegionsSubsystem::OnActorDeleted(AActor* InActor, bool bClearStencilIdValues)
{
	AColorCorrectRegion* AsRegion = Cast<AColorCorrectRegion>(InActor);
	if (AsRegion 
#if WITH_EDITORONLY_DATA
		&& !AsRegion->bIsEditorPreviewActor)
#else
		)
#endif
	{
#if WITH_EDITOR
		// In some cases, specifically in case when EndPlay is called or when CCA are part of a hidden sublevel
		// we don't want the stencil Ids to be reset.
		if (bClearStencilIdValues)
		{
			FColorCorrectRegionsStencilManager::OnCCRRemoved(GetWorld(), AsRegion);
		}
#endif
	}
}

void UColorCorrectRegionsSubsystem::OnDuplicateActorsEnd()
{
	bDuplicationStarted = false; 

	for (AActor* DuplicatedActor : DuplicatedActors)
	{
		if (AColorCorrectRegion* AsRegion = Cast<AColorCorrectRegion>(DuplicatedActor))
		{
			AssignNewPriorityIfNeeded(AsRegion, RegionsPriorityBased);
		}
		else
		{
			FColorCorrectRegionsStencilManager::CleanActor(DuplicatedActor);
		}
	}

	DuplicatedActors.Empty();
}

void UColorCorrectRegionsSubsystem::AssignStencilIdsToPerActorCC(AColorCorrectRegion* Region, bool bIgnoreUserNotificaion, bool bSoftAssign)
{
#if WITH_EDITOR
	if (!bSoftAssign && GEditor)
	{
		GEditor->BeginTransaction(LOCTEXT("PerActorCCActorAssigned", "Per actor CC Actor Assigned"));
	}
#endif
	FColorCorrectRegionsStencilManager::AssignStencilIdsToAllActorsForCCR(GetWorld(), Region, bIgnoreUserNotificaion, bSoftAssign);

#if WITH_EDITOR
	if (!bSoftAssign && GEditor)
	{
		this->Modify();
		GEditor->EndTransaction();
	}
#endif
}

void UColorCorrectRegionsSubsystem::ClearStencilIdsToPerActorCC(AColorCorrectRegion* Region)
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->BeginTransaction(LOCTEXT("PerActorCCActorRemoved", "Per actor CC Actor Removed"));
	}
#endif

	FColorCorrectRegionsStencilManager::RemoveStencilNumberForSelectedRegion(GetWorld(), Region);
	this->Modify();

#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->EndTransaction();
	}
#endif
}

void UColorCorrectRegionsSubsystem::CheckAssignedActorsValidity(AColorCorrectRegion* Region)
{
	FColorCorrectRegionsStencilManager::CheckAssignedActorsValidity(Region);
}

void UColorCorrectRegionsSubsystem::NotifyPerActorCCChanged(AColorCorrectRegion* Region)
{
#if WITH_EDITOR
	if (MultiUserManager)
	{
		MultiUserManager->SendPerActorAssignmentEvent(Region);
	}
#endif
}

void UColorCorrectRegionsSubsystem::RefreshRegions()
{
	RegionsPriorityBased.Reset();
	RegionsDistanceBased.Reset();
	for (TActorIterator<AColorCorrectRegion> It(GetWorld()); It; ++It)
	{
		AColorCorrectRegion* AsRegion = *It;
		if (IsRegionValid(AsRegion, GetWorld()))
		{
			if (!Cast<AColorCorrectionWindow>(AsRegion))
			{
				RegionsPriorityBased.Add(AsRegion);
			}
			else
			{
				RegionsDistanceBased.Add(AsRegion);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE