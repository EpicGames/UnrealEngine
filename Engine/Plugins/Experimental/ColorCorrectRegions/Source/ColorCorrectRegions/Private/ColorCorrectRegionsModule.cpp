// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorCorrectRegionsModule.h"

#include "ColorCorrectRegion.h"
#include "ColorCorrectWindow.h"

#if WITH_EDITOR
#include "Filters/CustomClassFilterData.h"
#include "LevelEditor.h"
#endif

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"

#define LOCTEXT_NAMESPACE "FColorCorrectRegionsModule"

void FColorCorrectRegionsModule::StartupModule()
{
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("ColorCorrectRegions"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/ColorCorrectRegionsShaders"), PluginShaderDir);

#if WITH_EDITOR
	RegisterOutlinerFilters();
#endif
}

void FColorCorrectRegionsModule::ShutdownModule()
{

}

#if WITH_EDITOR
void FColorCorrectRegionsModule::RegisterOutlinerFilters()
{
	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
	{
		if (const TSharedPtr<FFilterCategory> FilterCategory = LevelEditorModule->GetOutlinerFilterCategory(FLevelEditorOutlinerBuiltInCategories::VirtualProduction()))
		{
			const TSharedRef<FCustomClassFilterData> ColorCorrectionRegionActorClassData =
				MakeShared<FCustomClassFilterData>(AColorCorrectionRegion::StaticClass(), FilterCategory, FLinearColor::White);
			LevelEditorModule->AddCustomClassFilterToOutliner(ColorCorrectionRegionActorClassData);

			const TSharedRef<FCustomClassFilterData> ColorCorrectionWindowActorClassData =
				MakeShared<FCustomClassFilterData>(AColorCorrectionWindow::StaticClass(), FilterCategory, FLinearColor::White);
			LevelEditorModule->AddCustomClassFilterToOutliner(ColorCorrectionWindowActorClassData);
		}
	}
}
#endif

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FColorCorrectRegionsModule, ColorCorrectRegions);
DEFINE_LOG_CATEGORY(ColorCorrectRegions);
