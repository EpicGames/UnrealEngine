// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorCorrectRegionsPostProcessMaterial.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FCCRRegionDataInputParameter, "RegionData");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FCCRColorCorrectParameter, "ColorCorrectBase");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FCCRColorCorrectShadowsParameter, "ColorCorrectShadows");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FCCRColorCorrectMidtonesParameter, "ColorCorrectMidtones");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FCCRColorCorrectHighlightsParameter, "ColorCorrectHighlights");


IMPLEMENT_GLOBAL_SHADER(FColorCorrectRegionMaterialVS, "/ColorCorrectRegionsShaders/Private/ColorCorrectRegionsShader.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FColorCorrectRegionMaterialPS, "/ColorCorrectRegionsShaders/Private/ColorCorrectRegionsShader.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FColorCorrectWindowMaterialPS, "/ColorCorrectRegionsShaders/Private/ColorCorrectRegionsShader.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FColorCorrectScreenPassVS, "/ColorCorrectRegionsShaders/Private/ColorCorrectRegionsScreenPass.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FClearRectPS, "/ColorCorrectRegionsShaders/Private/ColorCorrectRegionsScreenPass.usf", "MainPS", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FCCRStencilMergerPS, "/ColorCorrectRegionsShaders/Private/ColorCorrectRegionsStencilMerger.usf", "MainPS", SF_Pixel);