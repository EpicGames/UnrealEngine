// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ColorCorrectRegion.h"
#include "ColorCorrectWindow.h"
#include "CoreMinimal.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalShader.h"
#include "MaterialShader.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Math/MathFwd.h"
#include "RHI.h"
#include "RHIStaticStates.h"
#include "RenderGraphResources.h"
#include "RenderGraphResources.h"
#include "RenderResource.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterUtils.h"
#include "ShaderPermutation.h"
#include "UObject/ConstructorHelpers.h"

// FScreenPassTextureViewportParameters and FScreenPassTextureInput
#include "ScreenPass.h"
#include "SceneTexturesConfig.h"



BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FCCRRegionDataInputParameter, )
	SHADER_PARAMETER(FVector3f, Rotate)
	SHADER_PARAMETER(FVector3f, Translate)
	SHADER_PARAMETER(FVector3f, Scale)

	SHADER_PARAMETER(float, WhiteTemp)
	SHADER_PARAMETER(float, Tint)
	SHADER_PARAMETER(float, Inner)
	SHADER_PARAMETER(float, Outer)
	SHADER_PARAMETER(float, Falloff)
	SHADER_PARAMETER(float, Intensity)
	SHADER_PARAMETER(float, FakeLight)
	SHADER_PARAMETER(uint32, ExcludeStencil)
	SHADER_PARAMETER(float, Invert)
END_GLOBAL_SHADER_PARAMETER_STRUCT()


BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FCCRColorCorrectParameter, )
	SHADER_PARAMETER(FVector4f, ColorSaturation)
	SHADER_PARAMETER(FVector4f, ColorContrast)
	SHADER_PARAMETER(FVector4f, ColorGamma)
	SHADER_PARAMETER(FVector4f, ColorGain)
	SHADER_PARAMETER(FVector4f, ColorOffset)
END_GLOBAL_SHADER_PARAMETER_STRUCT()


BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FCCRColorCorrectShadowsParameter, )
	SHADER_PARAMETER(FVector4f, ColorSaturation)
	SHADER_PARAMETER(FVector4f, ColorContrast)
	SHADER_PARAMETER(FVector4f, ColorGamma)
	SHADER_PARAMETER(FVector4f, ColorGain)
	SHADER_PARAMETER(FVector4f, ColorOffset)
	SHADER_PARAMETER(float, ShadowMax)
END_GLOBAL_SHADER_PARAMETER_STRUCT()


BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FCCRColorCorrectMidtonesParameter, )
	SHADER_PARAMETER(FVector4f, ColorSaturation)
	SHADER_PARAMETER(FVector4f, ColorContrast)
	SHADER_PARAMETER(FVector4f, ColorGamma)
	SHADER_PARAMETER(FVector4f, ColorGain)
	SHADER_PARAMETER(FVector4f, ColorOffset)
END_GLOBAL_SHADER_PARAMETER_STRUCT()


BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FCCRColorCorrectHighlightsParameter, )
	SHADER_PARAMETER(FVector4f, ColorSaturation)
	SHADER_PARAMETER(FVector4f, ColorContrast)
	SHADER_PARAMETER(FVector4f, ColorGamma)
	SHADER_PARAMETER(FVector4f, ColorGain)
	SHADER_PARAMETER(FVector4f, ColorOffset)
	SHADER_PARAMETER(float, HighlightsMin)
END_GLOBAL_SHADER_PARAMETER_STRUCT()


BEGIN_SHADER_PARAMETER_STRUCT(FCCRShaderInputParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, MergedStencilTexture)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT_REF(FWorkingColorSpaceShaderParameters, WorkingColorSpace)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, PostProcessOutput)
	SHADER_PARAMETER_STRUCT_ARRAY(FScreenPassTextureInput, PostProcessInput, [1])
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()


class FColorCorrectRegionsPostProcessMaterialShader : public FGlobalShader
{
public:
	using FParameters = FCCRShaderInputParameters;
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FColorCorrectRegionsPostProcessMaterialShader, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return !IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

};

class FColorCorrectRegionMaterialVS : public FColorCorrectRegionsPostProcessMaterialShader
{
public:
	DECLARE_GLOBAL_SHADER(FColorCorrectRegionMaterialVS);

	FColorCorrectRegionMaterialVS() = default;
	FColorCorrectRegionMaterialVS(const ShaderMetaType::CompiledShaderInitializerType & Initializer)
		: FColorCorrectRegionsPostProcessMaterialShader(Initializer)
	{}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View)
	{
		FGlobalShader::SetParameters<FViewUniformShaderParameters>(BatchedParameters, View.ViewUniformBuffer);
	}
};

class FColorCorrectGenericPS : public FColorCorrectRegionsPostProcessMaterialShader
{
public:
	enum class ETemperatureType : uint8
	{
		LegacyTemperature = static_cast<uint8>(EColorCorrectRegionTemperatureType::LegacyTemperature),
		WhiteBalance = static_cast<uint8>(EColorCorrectRegionTemperatureType::WhiteBalance),
		ColorTemperature = static_cast<uint8>(EColorCorrectRegionTemperatureType::ColorTemperature),
		Disabled,
		MAX
	};

	class FTemperatureType : SHADER_PERMUTATION_ENUM_CLASS("TEMPERATURE_TYPE", ETemperatureType);
	class FAdvancedShader : SHADER_PERMUTATION_BOOL("ADVANCED_CC");
	class FStencilEnabled : SHADER_PERMUTATION_BOOL("STENCIL_ENABLED");

	FColorCorrectGenericPS() = default;
	FColorCorrectGenericPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FColorCorrectRegionsPostProcessMaterialShader(Initializer)
	{}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View)
	{
		FGlobalShader::SetParameters<FViewUniformShaderParameters>(BatchedParameters, View.ViewUniformBuffer);
	}
};

class FColorCorrectRegionMaterialPS : public FColorCorrectGenericPS
{
public:

	DECLARE_GLOBAL_SHADER(FColorCorrectRegionMaterialPS);

	class FShaderType : SHADER_PERMUTATION_ENUM_CLASS("SHAPE_TYPE", EColorCorrectRegionsType);

	using FPermutationDomain = TShaderPermutationDomain<FShaderType, FTemperatureType, FAdvancedShader, FStencilEnabled>;

	FColorCorrectRegionMaterialPS() = default;
	FColorCorrectRegionMaterialPS(const ShaderMetaType::CompiledShaderInitializerType & Initializer)
		: FColorCorrectGenericPS(Initializer)
	{}
};


class FColorCorrectWindowMaterialPS : public FColorCorrectGenericPS
{
public:

	DECLARE_GLOBAL_SHADER(FColorCorrectWindowMaterialPS);

	class FShaderType : SHADER_PERMUTATION_ENUM_CLASS("WINDOW_TYPE", EColorCorrectWindowType);

	using FPermutationDomain = TShaderPermutationDomain<FShaderType, FTemperatureType, FAdvancedShader, FStencilEnabled>;

	FColorCorrectWindowMaterialPS() = default;
	FColorCorrectWindowMaterialPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FColorCorrectGenericPS(Initializer)
	{}
};

// The vertex shader used by DrawScreenPass to draw a rectangle.
class FColorCorrectScreenPassVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FColorCorrectScreenPassVS);

	FColorCorrectScreenPassVS() = default;
	FColorCorrectScreenPassVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

// A simple shader that outputs (0.,0.,0.,0.)
class FClearRectPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearRectPS);
	SHADER_USE_PARAMETER_STRUCT(FClearRectPS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FColorCorrectRegionMaterialPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("CLEAR_PS"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};


/** Vertex shader to rasterize obstructive CCR Plane. */
class FCCRStencilMergerVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCCRStencilMergerVS);

	FCCRStencilMergerVS() = default;
	FCCRStencilMergerVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

/** Pixel shader to rasterize obstructive CCR Plane. */
class FCCRStencilMergerPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCCRStencilMergerPS);
	SHADER_USE_PARAMETER_STRUCT(FCCRStencilMergerPS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return !IsMobilePlatform(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, StencilIds)
		SHADER_PARAMETER(uint32, StencilIdCount)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, PostProcessOutput)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
