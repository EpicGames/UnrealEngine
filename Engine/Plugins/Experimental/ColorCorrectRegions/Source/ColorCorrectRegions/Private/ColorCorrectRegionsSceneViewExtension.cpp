// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorCorrectRegionsSceneViewExtension.h"
#include "ColorCorrectRegionsModule.h"
#include "ColorCorrectRegionsSubsystem.h"
#include "ColorCorrectRegionsPostProcessMaterial.h"
#include "CommonRenderResources.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "DynamicResolutionState.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "FXRenderingUtils.h"
#include "PostProcess/PostProcessInputs.h"
#include "RHI.h"
#include "SceneRenderTargetParameters.h"
#include "SceneView.h"
#include "ScreenPass.h"

// Set this to 1 to clip pixels outside of bounding box.
#define CLIP_PIXELS_OUTSIDE_AABB 1

DECLARE_GPU_STAT_NAMED(ColorCorrectRegion, TEXT("ColorCorrectRegion"));

namespace
{
	FScreenPassTextureViewportParameters GetTextureViewportParameters(const FScreenPassTextureViewport& InViewport)
	{
		const FVector2f Extent(InViewport.Extent);
		const FVector2f ViewportMin(InViewport.Rect.Min.X, InViewport.Rect.Min.Y);
		const FVector2f ViewportMax(InViewport.Rect.Max.X, InViewport.Rect.Max.Y);
		const FVector2f ViewportSize = ViewportMax - ViewportMin;

		FScreenPassTextureViewportParameters Parameters;

		if (!InViewport.IsEmpty())
		{
			Parameters.Extent = FVector2f(Extent);
			Parameters.ExtentInverse = FVector2f(1.0f / Extent.X, 1.0f / Extent.Y);

			Parameters.ScreenPosToViewportScale = FVector2f(0.5f, -0.5f) * ViewportSize;	
			Parameters.ScreenPosToViewportBias = (0.5f * ViewportSize) + ViewportMin;	

			Parameters.ViewportMin = InViewport.Rect.Min;
			Parameters.ViewportMax = InViewport.Rect.Max;

			Parameters.ViewportSize = ViewportSize;
			Parameters.ViewportSizeInverse = FVector2f(1.0f / Parameters.ViewportSize.X, 1.0f / Parameters.ViewportSize.Y);

			Parameters.UVViewportMin = ViewportMin * Parameters.ExtentInverse;
			Parameters.UVViewportMax = ViewportMax * Parameters.ExtentInverse;

			Parameters.UVViewportSize = Parameters.UVViewportMax - Parameters.UVViewportMin;
			Parameters.UVViewportSizeInverse = FVector2f(1.0f / Parameters.UVViewportSize.X, 1.0f / Parameters.UVViewportSize.Y);

			Parameters.UVViewportBilinearMin = Parameters.UVViewportMin + 0.5f * Parameters.ExtentInverse;
			Parameters.UVViewportBilinearMax = Parameters.UVViewportMax - 0.5f * Parameters.ExtentInverse;
		}

		return Parameters;
	}


	void GetPixelSpaceBoundingRect(const FSceneView& InView, const FVector& InBoxCenter, const FVector& InBoxExtents, FIntRect& OutViewport, float& OutMaxDepth, float& OutMinDepth)
	{
		// Equivalent to D3D12_VIEWPORT_BOUNDS_MAX
		constexpr int32 CMaxViewportDimension = 32767;

		OutViewport = FIntRect(CMaxViewportDimension, CMaxViewportDimension, -CMaxViewportDimension, -CMaxViewportDimension);

		// 8 corners of the bounding box. To be multiplied by box extent and offset by the center.
		const int NumCorners = 8;
		const FVector Verts[NumCorners] = {
			FVector(1, 1, 1),
			FVector(1, 1,-1),
			FVector(1,-1, 1),
			FVector(1,-1,-1),
			FVector(-1, 1, 1),
			FVector(-1, 1,-1),
			FVector(-1,-1, 1),
			FVector(-1,-1,-1) };

		for (int32 Index = 0; Index < NumCorners; Index++)
		{
			// Project bounding box vertecies into screen space.
			const FVector WorldVert = InBoxCenter + (Verts[Index] * InBoxExtents);
			FVector2D PixelVert;
			FVector4 ScreenSpaceCoordinate = InView.WorldToScreen(WorldVert);

			OutMaxDepth = FMath::Max<float>(ScreenSpaceCoordinate.W, OutMaxDepth);
			OutMinDepth = FMath::Min<float>(ScreenSpaceCoordinate.W, OutMinDepth);

			if (InView.ScreenToPixel(ScreenSpaceCoordinate, PixelVert))
			{
				// Update screen-space bounding box with with transformed vert.
				OutViewport.Min.X = FMath::Min<int32>(OutViewport.Min.X, PixelVert.X);
				OutViewport.Min.Y = FMath::Min<int32>(OutViewport.Min.Y, PixelVert.Y);

				OutViewport.Max.X = FMath::Max<int32>(OutViewport.Max.X, PixelVert.X);
				OutViewport.Max.Y = FMath::Max<int32>(OutViewport.Max.Y, PixelVert.Y);
			}
		}
	}

	// Function that calculates all points of intersection between plane and bounding box. Resulting points are unsorted.
	void CalculatePlaneAABBIntersectionPoints(const FPlane& Plane, const FVector& BoxCenter, const FVector& BoxExtents, TArray<FVector>& OutPoints)
	{
		const FVector MaxCorner = BoxCenter + BoxExtents;

		const FVector Verts[3][4] = {
			{
				// X Direction
				FVector(-1, -1, -1),
				FVector(-1,  1, -1),
				FVector(-1, -1,  1),
				FVector(-1,  1,  1),
			},
			{
				// Y Direction
				FVector(-1, -1, -1),
				FVector( 1, -1, -1),
				FVector( 1, -1,  1),
				FVector(-1, -1,  1),
			},
			{
				// Z Direction
				FVector(-1, -1, -1),
				FVector( 1, -1, -1),
				FVector( 1,  1, -1),
				FVector(-1,  1, -1),
			}
		};

		FVector Intersection;
		FVector Start;
		FVector End;

		for (int RunningAxis_Dir = 0; RunningAxis_Dir < 3; RunningAxis_Dir++)
		{
			const FVector *CornerLocations = Verts[RunningAxis_Dir];
			for (int RunningCorner = 0; RunningCorner < 4; RunningCorner++)
			{
				Start = BoxCenter + BoxExtents * CornerLocations[RunningCorner];
				End = FVector(Start.X, Start.Y, Start.Z);
				End[RunningAxis_Dir] = MaxCorner[RunningAxis_Dir];
				if (FMath::SegmentPlaneIntersection(Start, End, Plane, Intersection))
				{
					OutPoints.Add(Intersection);
				}
			}
		}
	}

	// Takes in an existing viewport and updates it with an intersection bounding rectangle.
	void UpdateMinMaxWithFrustrumAABBIntersection(const FSceneView& InView, const FVector& InBoxCenter, const FVector& InBoxExtents, FIntRect& OutViewportToUpdate, float& OutMaxDepthToUpdate)
	{
		TArray<FVector> Points;
		Points.Reserve(6);
		static bool bNotifiedOfClippingPlaneError = false;

		if (InView.bHasNearClippingPlane)
		{
			CalculatePlaneAABBIntersectionPoints(InView.NearClippingPlane, InBoxCenter, InBoxExtents, Points);
		}
		// Previously last plane was near clipping plane.
		else if (InView.ViewFrustum.Planes.Num() == 5)
		{
			CalculatePlaneAABBIntersectionPoints(InView.ViewFrustum.Planes[4], InBoxCenter, InBoxExtents, Points);
		}
		else if (!bNotifiedOfClippingPlaneError)
		{
			bNotifiedOfClippingPlaneError = true;
			UE_LOGF(ColorCorrectRegions, Error, "Couldn't find a correct near clipping plane in View Frustrum");
		}

		if (Points.Num() == 0)
		{
			return;
		}

		for (FVector Point : Points)
		{
			// Project bounding box vertecies into screen space.
			FVector4 ScreenSpaceCoordinate = InView.WorldToScreen(Point);
			FVector4 ScreenSpaceCoordinateScaled = ScreenSpaceCoordinate * 1.0 / ScreenSpaceCoordinate.W;

			OutMaxDepthToUpdate = FMath::Max<float>(ScreenSpaceCoordinate.W, OutMaxDepthToUpdate);
			FVector2D PixelVert;

			if (InView.ScreenToPixel(ScreenSpaceCoordinate, PixelVert))
			{
				// Update screen-space bounding box with with transformed vert.
				OutViewportToUpdate.Min.X = FMath::Min<int32>(OutViewportToUpdate.Min.X, PixelVert.X);
				OutViewportToUpdate.Min.Y = FMath::Min<int32>(OutViewportToUpdate.Min.Y, PixelVert.Y);

				OutViewportToUpdate.Max.X = FMath::Max<int32>(OutViewportToUpdate.Max.X, PixelVert.X);
				OutViewportToUpdate.Max.Y = FMath::Max<int32>(OutViewportToUpdate.Max.Y, PixelVert.Y);
			}
		}
	}

	bool ViewSupportsRegions(const FSceneView& View)
	{
		return View.Family->EngineShowFlags.PostProcessing &&
				View.Family->EngineShowFlags.PostProcessMaterial;
	}

	// A helper function for getting the right shader for SDF based CCRs.
	TShaderMapRef<FColorCorrectRegionMaterialPS> GetRegionShader(const FGlobalShaderMap* GlobalShaderMap, EColorCorrectRegionsType RegionType, FColorCorrectGenericPS::ETemperatureType TemperatureType, bool bIsAdvanced, bool bUseStencil)
	{
		FColorCorrectRegionMaterialPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FColorCorrectGenericPS::FAdvancedShader>(bIsAdvanced);
		PermutationVector.Set<FColorCorrectGenericPS::FStencilEnabled>(bUseStencil);
		PermutationVector.Set<FColorCorrectGenericPS::FTemperatureType>(TemperatureType);
		PermutationVector.Set<FColorCorrectRegionMaterialPS::FShaderType>(static_cast<EColorCorrectRegionsType>(FMath::Min(static_cast<int32>(RegionType), static_cast<int32>(EColorCorrectRegionsType::MAX) - 1)));

		return TShaderMapRef<FColorCorrectRegionMaterialPS>(GlobalShaderMap, PermutationVector);
	}

	// A helper function for getting the right shader for distance based CCRs.
	TShaderMapRef<FColorCorrectWindowMaterialPS> GetWindowShader(const FGlobalShaderMap* GlobalShaderMap, EColorCorrectWindowType RegionType, FColorCorrectGenericPS::ETemperatureType TemperatureType, bool bIsAdvanced, bool bUseStencil)
	{
		FColorCorrectWindowMaterialPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FColorCorrectGenericPS::FAdvancedShader>(bIsAdvanced);
		PermutationVector.Set<FColorCorrectGenericPS::FStencilEnabled>(bUseStencil);
		PermutationVector.Set<FColorCorrectGenericPS::FTemperatureType>(TemperatureType);
		PermutationVector.Set<FColorCorrectWindowMaterialPS::FShaderType>(static_cast<EColorCorrectWindowType>(FMath::Min(static_cast<int32>(RegionType), static_cast<int32>(EColorCorrectWindowType::MAX) - 1)));

		return TShaderMapRef<FColorCorrectWindowMaterialPS>(GlobalShaderMap, PermutationVector);
	}

	FVector4 Clamp(const FVector4 & VectorToClamp, float Min, float Max)
	{
		return FVector4(FMath::Clamp(VectorToClamp.X, Min, Max),
						FMath::Clamp(VectorToClamp.Y, Min, Max),
						FMath::Clamp(VectorToClamp.Z, Min, Max),
						FMath::Clamp(VectorToClamp.W, Min, Max));
	}


	void StencilMerger
		( FRDGBuilder& GraphBuilder
		, const FGlobalShaderMap* GlobalShaderMap
		, const FScreenPassRenderTarget& SceneColorRenderTarget
		, const FSceneView& View
		, const FScreenPassTextureViewportParameters& SceneTextureViewportParams
		, const FScreenPassTextureViewport& RegionViewport
		, const FSceneTextureShaderParameters& SceneTextures
		, const TArray<uint32>& StencilIds
		, FScreenPassRenderTarget& OutMergedStencilRenderTarget)
	{
		if (StencilIds.Num() == 0)
		{
			return;
		}
		FRDGTextureDesc DepthBufferOutputDesc = SceneColorRenderTarget.Texture->Desc;
		DepthBufferOutputDesc.Format = EPixelFormat::PF_DepthStencil;
		DepthBufferOutputDesc.ClearValue = FClearValueBinding(0);
		DepthBufferOutputDesc.Flags = TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable;
		DepthBufferOutputDesc.ClearValue = FClearValueBinding(0, 0);

		FRDGTextureDesc Desc = SceneColorRenderTarget.Texture->Desc;
		Desc.Format = EPixelFormat::PF_R8_UINT;
		FRDGTexture* MergedStencilTexture = GraphBuilder.CreateTexture(Desc, TEXT("CCR_MergedStencil"));
		OutMergedStencilRenderTarget = FScreenPassRenderTarget(MergedStencilTexture, SceneColorRenderTarget.ViewRect, ERenderTargetLoadAction::EClear);
		{
			TShaderMapRef<FCCRStencilMergerPS> StencilMergerPS(GlobalShaderMap);
			TShaderMapRef<FColorCorrectScreenPassVS> StencilMergerVS(GlobalShaderMap);
			FCCRStencilMergerPS::FParameters* Parameters = GraphBuilder.AllocParameters<FCCRStencilMergerPS::FParameters>();
			Parameters->SceneTextures = SceneTextures;
			Parameters->RenderTargets[0] = OutMergedStencilRenderTarget.GetRenderTargetBinding();
			Parameters->PostProcessOutput = SceneTextureViewportParams;
			Parameters->View = View.ViewUniformBuffer;

			Parameters->StencilIds = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(CreateStructuredBuffer(GraphBuilder, TEXT("CCR.StencilIdBuffer"), sizeof(uint32), StencilIds.Num(), &StencilIds[0], sizeof(uint32) * StencilIds.Num())));
			Parameters->StencilIdCount = StencilIds.Num();

			AddDrawScreenPass(
				GraphBuilder,
				RDG_EVENT_NAME("ColorCorrectRegions_StencilMerger"),
				View,
				RegionViewport,
				RegionViewport,
				StencilMergerVS,
				StencilMergerPS,
				Parameters
			);
		}
	}

	void SortRegionProxiesByDistance(TArray<FColorCorrectRenderProxyPtr>& ProxiesDistanceBased, const FVector& ViewLocation)
	{
		check(IsInRenderingThread());

		TMap<FColorCorrectRenderProxyPtr, double> DistanceMap;
		for (FColorCorrectRenderProxyPtr State : ProxiesDistanceBased)
		{
			FVector CameraToRegionVec = (State->BoxOrigin - ViewLocation);
			DistanceMap.Add(State, CameraToRegionVec.Dot(CameraToRegionVec));
		}

		ProxiesDistanceBased.Sort([&DistanceMap](const FColorCorrectRenderProxyPtr& A, const FColorCorrectRenderProxyPtr& B) {
			// Regions with the same distance could potentially cause flickering on overlap
			return DistanceMap[A] > DistanceMap[B];
		});

	}

	bool RenderRegion
		( FRDGBuilder& GraphBuilder
		, const FSceneView& View
		, const FSceneViewFamily& ViewFamily
		, FColorCorrectRenderProxyPtr RegionState
		, const FIntRect& PrimaryViewRect
		, const FScreenPassRenderTarget& SceneColorRenderTarget
		, const float ScreenPercentage
		, FScreenPassRenderTarget& BackBufferRenderTarget
		, const FScreenPassTextureViewportParameters& SceneTextureViewportParams
		, const FScreenPassTextureInput& SceneTextureInput
		, const FSceneTextureShaderParameters& SceneTextures
		, FGlobalShaderMap* GlobalShaderMap
		, FRHIBlendState* DefaultBlendState)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, ColorCorrectRegion, "ColorCorrectRegion");

		FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

		/* If Region is pending for kill, invisible or disabled we don't need to render it.
		*	If Region's Primitive is not visible in the current view's scene then we don't need to render it either.
		*	We are checking if the region belongs to the same world as the view.
		*/
		if (!RegionState->bIsActiveThisFrame ||
			View.HiddenPrimitives.Contains(RegionState->FirstPrimitiveId) ||
			(View.ShowOnlyPrimitives.IsSet() && !View.ShowOnlyPrimitives->Contains(RegionState->FirstPrimitiveId))
			)
		{
			return false;
		}

		// If bounding box is zero, then we don't need to do anything.
		if (RegionState->BoxExtent.IsNearlyZero())
		{
			return false;
		}

		FIntRect BoundingRectangle;

		float MaxDepth = -BIG_NUMBER;
		float MinDepth = BIG_NUMBER;

		if (RegionState->Invert)
		{
			// In case of Region inversion we would to render the entire screen
			BoundingRectangle = PrimaryViewRect;
		}
		else
		{
			GetPixelSpaceBoundingRect(View, RegionState->BoxOrigin, RegionState->BoxExtent, BoundingRectangle, MaxDepth, MinDepth);

			// This is to handle corner cases when user has a very long disproportionate region and gets either
			// within bounds or close to the center.
			float MaxBoxExtent = FMath::Abs(RegionState->BoxExtent.GetMax());
			if (MaxDepth >= 0 && MinDepth < 0)
			{
				UpdateMinMaxWithFrustrumAABBIntersection(View, RegionState->BoxOrigin, RegionState->BoxExtent, BoundingRectangle, MaxDepth);
			}

			FIntRect ConstrainedViewRect = View.UnscaledViewRect;

			// We need to make sure that Bounding Rectangle is offset by the position of the View's Viewport.
			BoundingRectangle.Min -= ConstrainedViewRect.Min;
			UE::Math::TIntRect<int64> BoundingRectToBeTruncated(BoundingRectangle);

			// As of UE5.5 Scale method has non templated 64 bit narrowing conversion check even if FIntRect is 32 bit. 
			BoundingRectToBeTruncated = BoundingRectToBeTruncated.Scale(ScreenPercentage);

			// Culling all regions that are not within the screen bounds on a 64 bit viewport.
			if ((BoundingRectToBeTruncated.Min.X >= PrimaryViewRect.Width() ||
				BoundingRectToBeTruncated.Min.Y >= PrimaryViewRect.Height() ||
				BoundingRectToBeTruncated.Max.X <= 0.0f ||
				BoundingRectToBeTruncated.Max.Y <= 0.0f ||
				MaxDepth < 0.0f))
			{
				return false;
			}

			// Truncate back to 32 bit. At this point CC Actors with bounding rectangles that do not fit the viewport are skipped.
			BoundingRectangle = FIntRect(
				FIntPoint((int32)BoundingRectToBeTruncated.Min.X, (int32)BoundingRectToBeTruncated.Min.Y),
				FIntPoint((int32)BoundingRectToBeTruncated.Max.X, (int32)BoundingRectToBeTruncated.Max.Y)
				);

			// Clipping is required because as we get closer to the bounding box the bounds
			// May extend beyond Allowed render target size.
			BoundingRectangle.Clip(PrimaryViewRect);

			// Equivalent to D3D12_VIEWPORT_BOUNDS_MAX
			constexpr int32 CMaxViewportDimension = 32767;

			// Equivalent to D3D12_VIEWPORT_BOUNDS_MIN
			constexpr int32 CMinViewportDimension = -32768;

			// Clamp the bounding rectangle to the max viewport dimensions.
			BoundingRectangle.Min.X = FMath::Clamp(BoundingRectangle.Min.X, CMinViewportDimension, CMaxViewportDimension);
			BoundingRectangle.Min.Y = FMath::Clamp(BoundingRectangle.Min.Y, CMinViewportDimension, CMaxViewportDimension);
			BoundingRectangle.Max.X = FMath::Clamp(BoundingRectangle.Max.X, CMinViewportDimension, CMaxViewportDimension);
			BoundingRectangle.Max.Y = FMath::Clamp(BoundingRectangle.Max.Y, CMinViewportDimension, CMaxViewportDimension);

			// Check if CCR is too small to be rendered (less than one pixel on the screen).
			if (BoundingRectangle.Width() == 0 || BoundingRectangle.Height() == 0)
			{
				return false;
			}
		}


		bool bIsAdvanced = false;

		const FVector4 One(1., 1., 1., 1.);
		const FVector4 Zero(0., 0., 0., 0.);
		TArray<const FColorGradePerRangeSettings*> AdvancedSettings{ &RegionState->ColorGradingSettings.Shadows,
																&RegionState->ColorGradingSettings.Midtones,
																&RegionState->ColorGradingSettings.Highlights };

		// Check if any of the regions are advanced.
		for (auto SettingsIt = AdvancedSettings.CreateConstIterator(); SettingsIt; ++SettingsIt)
		{
			const FColorGradePerRangeSettings* ColorGradingSettings = *SettingsIt;
			if (!ColorGradingSettings->Saturation.Equals(One, SMALL_NUMBER) ||
				!ColorGradingSettings->Contrast.Equals(One, SMALL_NUMBER) ||
				!ColorGradingSettings->Gamma.Equals(One, SMALL_NUMBER) ||
				!ColorGradingSettings->Gain.Equals(One, SMALL_NUMBER) ||
				!ColorGradingSettings->Offset.Equals(Zero, SMALL_NUMBER))
			{
				bIsAdvanced = true;
				break;
			}
		}

		const FScreenPassTextureViewport RegionViewport(SceneColorRenderTarget.Texture, BoundingRectangle);

		FCCRShaderInputParameters* PostProcessMaterialParameters = GraphBuilder.AllocParameters<FCCRShaderInputParameters>();
		PostProcessMaterialParameters->RenderTargets[0] = BackBufferRenderTarget.GetRenderTargetBinding();

		PostProcessMaterialParameters->WorkingColorSpace = GDefaultWorkingColorSpaceUniformBuffer.GetUniformBufferRef();
		PostProcessMaterialParameters->PostProcessOutput = SceneTextureViewportParams;
		PostProcessMaterialParameters->PostProcessInput[0] = SceneTextureInput;
		PostProcessMaterialParameters->SceneTextures = SceneTextures;
		PostProcessMaterialParameters->View = View.ViewUniformBuffer;

		TShaderMapRef<FColorCorrectRegionMaterialVS> VertexShader(GlobalShaderMap);
		const float DefaultTemperature = 6500;
		const float DefaultTint = 0;

		// If temperature is default we don't want to do the calculations.
		FColorCorrectRegionMaterialPS::ETemperatureType TemperatureType = FMath::IsNearlyEqual(RegionState->Temperature, DefaultTemperature) && FMath::IsNearlyEqual(RegionState->Tint, DefaultTint)
			? FColorCorrectRegionMaterialPS::ETemperatureType::Disabled
			: static_cast<FColorCorrectRegionMaterialPS::ETemperatureType>(RegionState->TemperatureType);

		FScreenPassRenderTarget MergedStencilRenderTarget;
		if (RegionState->bEnablePerActorCC)
		{
			TArray<uint32> StencilIds = RegionState->StencilIds;
			StencilMerger(GraphBuilder, GlobalShaderMap, SceneColorRenderTarget, View, SceneTextureViewportParams, RegionViewport, SceneTextures, StencilIds, MergedStencilRenderTarget);
		}

		TShaderRef<FColorCorrectGenericPS> PixelShader;
		if (RegionState->ProxyType == FColorCorrectRenderProxy::DistanceBased)
		{
			PixelShader = GetWindowShader(GlobalShaderMap, RegionState->WindowType, TemperatureType, bIsAdvanced, MergedStencilRenderTarget.IsValid());
		}
		else if (RegionState->ProxyType == FColorCorrectRenderProxy::PriorityBased)
		{
			PixelShader = GetRegionShader(GlobalShaderMap, RegionState->Type, TemperatureType, bIsAdvanced, MergedStencilRenderTarget.IsValid());
		}

		if (MergedStencilRenderTarget.IsValid())
		{
			PostProcessMaterialParameters->MergedStencilTexture = MergedStencilRenderTarget.Texture;
		}

		ClearUnusedGraphResources(VertexShader, PixelShader, PostProcessMaterialParameters);

		FCCRRegionDataInputParameter RegionData;
		FCCRColorCorrectParameter CCBase;
		FCCRColorCorrectShadowsParameter CCShadows;
		FCCRColorCorrectMidtonesParameter CCMidtones;
		FCCRColorCorrectHighlightsParameter CCHighlights;
		
		// Setting constant buffer data to be passed to the shader.
		{
			RegionData.Rotate = FMath::DegreesToRadians<FVector3f>(RegionState->ActorRotation);
			RegionData.Translate = RegionState->ActorLocation;

			const float ScaleMultiplier = View.WorldToMetersScale / 2.;
			// Pre multiplied scale. 
			RegionData.Scale = (FVector3f)RegionState->ActorScale * ScaleMultiplier;

			RegionData.WhiteTemp = RegionState->Temperature;
			RegionData.Tint = RegionState->Tint;

			RegionData.Inner = RegionState->Inner;
			RegionData.Outer = RegionState->Outer;

			RegionData.Falloff = RegionState->Falloff;
			RegionData.Intensity = RegionState->Intensity;
			RegionData.Invert = RegionState->Invert;
			RegionData.ExcludeStencil = static_cast<uint32>(RegionState->PerActorColorCorrection);

			CCBase.ColorSaturation = (FVector4f)RegionState->ColorGradingSettings.Global.Saturation;
			CCBase.ColorContrast = (FVector4f)RegionState->ColorGradingSettings.Global.Contrast;
			CCBase.ColorGamma = (FVector4f)RegionState->ColorGradingSettings.Global.Gamma;
			CCBase.ColorGain = (FVector4f)RegionState->ColorGradingSettings.Global.Gain;
			CCBase.ColorOffset = (FVector4f)RegionState->ColorGradingSettings.Global.Offset;

			// Set advanced 
			if (bIsAdvanced)
			{
				const float GammaMin = 0.02f;
				const float GammaMax = 10.f;
				//clamp(ExternalExpressions.ColorGammaHighlights, 0.02, 10.)
				CCShadows.ColorSaturation = (FVector4f)RegionState->ColorGradingSettings.Shadows.Saturation;
				CCShadows.ColorContrast = (FVector4f)RegionState->ColorGradingSettings.Shadows.Contrast;
				CCShadows.ColorGamma = (FVector4f)Clamp(RegionState->ColorGradingSettings.Shadows.Gamma, GammaMin, GammaMax);
				CCShadows.ColorGain = (FVector4f)RegionState->ColorGradingSettings.Shadows.Gain;
				CCShadows.ColorOffset = (FVector4f)RegionState->ColorGradingSettings.Shadows.Offset;
				CCShadows.ShadowMax = RegionState->ColorGradingSettings.ShadowsMax;

				CCMidtones.ColorSaturation = (FVector4f)RegionState->ColorGradingSettings.Midtones.Saturation;
				CCMidtones.ColorContrast = (FVector4f)RegionState->ColorGradingSettings.Midtones.Contrast;
				CCMidtones.ColorGamma = (FVector4f)Clamp(RegionState->ColorGradingSettings.Midtones.Gamma, GammaMin, GammaMax);
				CCMidtones.ColorGain = (FVector4f)RegionState->ColorGradingSettings.Midtones.Gain;
				CCMidtones.ColorOffset = (FVector4f)RegionState->ColorGradingSettings.Midtones.Offset;

				CCHighlights.ColorSaturation = (FVector4f)RegionState->ColorGradingSettings.Highlights.Saturation;
				CCHighlights.ColorContrast = (FVector4f)RegionState->ColorGradingSettings.Highlights.Contrast;
				CCHighlights.ColorGamma = (FVector4f)Clamp(RegionState->ColorGradingSettings.Highlights.Gamma, GammaMin, GammaMax);
				CCHighlights.ColorGain = (FVector4f)RegionState->ColorGradingSettings.Highlights.Gain;
				CCHighlights.ColorOffset = (FVector4f)RegionState->ColorGradingSettings.Highlights.Offset;
				CCHighlights.HighlightsMin = RegionState->ColorGradingSettings.HighlightsMin;
			}

		}

#if CLIP_PIXELS_OUTSIDE_AABB
		// In case this is a second pass we need to clear the viewport in the backbuffer texture.
		// We don't need to clear the entire texture, just the render viewport.
		if (BackBufferRenderTarget.LoadAction == ERenderTargetLoadAction::ELoad)
		{
			FClearRectPS::FParameters* Parameters = GraphBuilder.AllocParameters<FClearRectPS::FParameters>();
			TShaderMapRef<FClearRectPS> CopyPixelShader(GlobalShaderMap);
			TShaderMapRef<FColorCorrectScreenPassVS> ScreenPassVS(GlobalShaderMap);
			Parameters->RenderTargets[0] = BackBufferRenderTarget.GetRenderTargetBinding();

			AddDrawScreenPass(
				GraphBuilder,
				RDG_EVENT_NAME("ColorCorrectRegions_ClearViewport"),
				View,
				RegionViewport,
				RegionViewport,
				ScreenPassVS,
				CopyPixelShader,
				DefaultBlendState,
				Parameters
			);
		}
#endif
		// Main region rendering.
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("ColorCorrectRegions"),
			PostProcessMaterialParameters,
			ERDGPassFlags::Raster,
			[&View,
			RegionViewport,
			VertexShader,
			PixelShader,
			DefaultBlendState,
			DepthStencilState,
			PostProcessMaterialParameters,
			RegionData,
			CCBase,
			CCShadows,
			CCMidtones,
			CCHighlights,
			bIsAdvanced,
			MergedStencilRenderTarget](FRHICommandList& RHICmdList)
			{
				DrawScreenPass(
					RHICmdList,
					View,
					RegionViewport, // Output Viewport
					RegionViewport, // Input Viewport
					FScreenPassPipelineState(VertexShader, PixelShader, DefaultBlendState, DepthStencilState),
					EScreenPassDrawFlags::None,
					[&](FRHICommandList& RHICmdList)
					{
						FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

						SetUniformBufferParameterImmediate(BatchedParameters, PixelShader->GetUniformBufferParameter<FCCRRegionDataInputParameter>(), RegionData);
						SetUniformBufferParameterImmediate(BatchedParameters, PixelShader->GetUniformBufferParameter<FCCRColorCorrectParameter>(), CCBase);
						if (bIsAdvanced)
						{
							SetUniformBufferParameterImmediate(BatchedParameters, PixelShader->GetUniformBufferParameter<FCCRColorCorrectShadowsParameter>(), CCShadows);
							SetUniformBufferParameterImmediate(BatchedParameters, PixelShader->GetUniformBufferParameter<FCCRColorCorrectMidtonesParameter>(), CCMidtones);
							SetUniformBufferParameterImmediate(BatchedParameters, PixelShader->GetUniformBufferParameter<FCCRColorCorrectHighlightsParameter>(), CCHighlights);
						}

						PixelShader->SetParameters(BatchedParameters, View);
						SetShaderParameters(BatchedParameters, PixelShader, *PostProcessMaterialParameters);

						RHICmdList.SetBatchedShaderParameters(PixelShader.GetPixelShader(), BatchedParameters);

						BatchedParameters.Reset();

						VertexShader->SetParameters(BatchedParameters, View);
						SetShaderParameters(BatchedParameters, VertexShader, *PostProcessMaterialParameters);

						RHICmdList.SetBatchedShaderParameters(VertexShader.GetVertexShader(), BatchedParameters);
					});

			});

		// Since we've rendered into the backbuffer already we have to use load flag instead.
		BackBufferRenderTarget.LoadAction = ERenderTargetLoadAction::ELoad;

		FCopyRectPS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
		Parameters->InputTexture = BackBufferRenderTarget.Texture;
		Parameters->InputSampler = TStaticSamplerState<>::GetRHI();
		Parameters->RenderTargets[0] = SceneColorRenderTarget.GetRenderTargetBinding();

		TShaderMapRef<FCopyRectPS> CopyPixelShader(GlobalShaderMap);
		TShaderMapRef<FColorCorrectScreenPassVS> ScreenPassVS(GlobalShaderMap);

#if CLIP_PIXELS_OUTSIDE_AABB
		// Blending the output from the main step with scene color.
		// src.rgb*src.a + dest.rgb*(1.-src.a); alpha = src.a*0. + dst.a*1.0
		FRHIBlendState* CopyBlendState = TStaticBlendState<CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
#else	
		FRHIBlendState* CopyBlendState = DefaultBlendState;
#endif
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("ColorCorrectRegions_CopyViewport"),
			Parameters,
			ERDGPassFlags::Raster,
			[&View, ScreenPassVS, CopyPixelShader, RegionViewport, Parameters, CopyBlendState](FRHICommandList& RHICmdList)
			{
				DrawScreenPass(
					RHICmdList,
					View,
					RegionViewport,
					RegionViewport,
					FScreenPassPipelineState(ScreenPassVS, CopyPixelShader, CopyBlendState),
					EScreenPassDrawFlags::None,
					[&](FRHICommandList&)
					{
						SetShaderParameters(RHICmdList, CopyPixelShader, CopyPixelShader.GetPixelShader(), *Parameters);
					});
			});

		return true;

	}

}

FColorCorrectRegionsSceneViewExtension::FColorCorrectRegionsSceneViewExtension(const FAutoRegister& AutoRegister, UWorld* InWorld,  UColorCorrectRegionsSubsystem* InWorldSubsystem) :
	FWorldSceneViewExtension(AutoRegister, InWorld), WorldSubsystem(InWorldSubsystem)
{
}

void FColorCorrectRegionsSceneViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	if (IsValid(WorldSubsystem))
	{
		WorldSubsystem->TransferStates();
	}
}

void FColorCorrectRegionsSceneViewExtension::PrePostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessingInputs& Inputs)
{
	if (IsValid(WorldSubsystem))
	{
		// Necessary for when an actor is added or removed from the scene. Also when priority is changed.
		if ((WorldSubsystem->ProxiesPriorityBased.Num() == 0 && WorldSubsystem->ProxiesDistanceBased.Num() == 0) || !ViewSupportsRegions(View))
		{
			return;
		}
	}
	else
	{
		return;
	}

	Inputs.Validate();

	const FSceneViewFamily& ViewFamily = *View.Family;

	const FIntRect PrimaryViewRect = UE::FXRenderingUtils::GetRawViewRectUnsafe(View);

	// Take viewport scaling into account
	const float ScreenPercentage = (View.UnscaledViewRect.Width() > 0) ? static_cast<float>(PrimaryViewRect.Width()) / static_cast<float>(View.UnscaledViewRect.Width()) : ViewFamily.SecondaryViewFraction;

	FScreenPassTexture SceneColor((*Inputs.SceneTextures)->SceneColorTexture, PrimaryViewRect);

	if (!SceneColor.IsValid())
	{
		return;
	}

	{
		// Getting material data for the current view.
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		// Reusing the same output description for our back buffer as SceneColor
		FRDGTextureDesc ColorCorrectRegionsOutputDesc = SceneColor.Texture->Desc;

		ColorCorrectRegionsOutputDesc.Format = PF_FloatRGBA;
		FLinearColor ClearColor(0., 0., 0., 0.);
		ColorCorrectRegionsOutputDesc.ClearValue = FClearValueBinding(ClearColor);

		FRDGTexture* BackBufferRenderTargetTexture = GraphBuilder.CreateTexture(ColorCorrectRegionsOutputDesc, TEXT("BackBufferRenderTargetTexture"));
		FScreenPassRenderTarget BackBufferRenderTarget = FScreenPassRenderTarget(BackBufferRenderTargetTexture, SceneColor.ViewRect, ERenderTargetLoadAction::EClear);
		FScreenPassRenderTarget SceneColorRenderTarget(SceneColor, ERenderTargetLoadAction::ELoad);
		const FScreenPassTextureViewport SceneColorTextureViewport(SceneColor);

		FRHIBlendState* DefaultBlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();

		RDG_EVENT_SCOPE(GraphBuilder, "Color Correct Regions %dx%d", SceneColorTextureViewport.Rect.Width(), SceneColorTextureViewport.Rect.Height());

		FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		const FScreenPassTextureViewportParameters SceneTextureViewportParams = GetTextureViewportParameters(SceneColorTextureViewport);
		FScreenPassTextureInput SceneTextureInput;
		{
			SceneTextureInput.Viewport = SceneTextureViewportParams;
			SceneTextureInput.Texture = SceneColorRenderTarget.Texture;
			SceneTextureInput.Sampler = PointClampSampler;
		}

		// Because we are not using proxy material, but plain global shader, we need to setup Scene textures ourselves.
		// We don't need to do this per region.
		FSceneTextureShaderParameters SceneTextures = CreateSceneTextureShaderParameters(GraphBuilder, View, ESceneTextureSetupMode::All);

		{
			for (auto It = WorldSubsystem->ProxiesPriorityBased.CreateConstIterator(); It; ++It)
			{
				FColorCorrectRenderProxyPtr Region = *It;
				RenderRegion(GraphBuilder
					, View
					, ViewFamily
					, Region
					, PrimaryViewRect
					, SceneColorRenderTarget
					, ScreenPercentage
					, BackBufferRenderTarget
					, SceneTextureViewportParams
					, SceneTextureInput
					, SceneTextures
					, GlobalShaderMap
					, DefaultBlendState);
			}
			SortRegionProxiesByDistance(WorldSubsystem->ProxiesDistanceBased, View.ViewLocation);
			for (auto It = WorldSubsystem->ProxiesDistanceBased.CreateConstIterator(); It; ++It)
			{
				FColorCorrectRenderProxyPtr Region = *It;
				RenderRegion(GraphBuilder
					, View
					, ViewFamily
					, Region
					, PrimaryViewRect
					, SceneColorRenderTarget
					, ScreenPercentage
					, BackBufferRenderTarget
					, SceneTextureViewportParams
					, SceneTextureInput
					, SceneTextures
					, GlobalShaderMap
					, DefaultBlendState);
			}
		}
	}
}

void FColorCorrectRegionsSceneViewExtension::Invalidate()
{
	WorldSubsystem = nullptr;
}
