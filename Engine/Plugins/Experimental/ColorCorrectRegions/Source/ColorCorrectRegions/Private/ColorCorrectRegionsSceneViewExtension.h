// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "SceneViewExtension.h"
#include "RHI.h"
#include "RHIResources.h"

class UColorCorrectRegionsSubsystem;
class UMaterialInterface;
class FRDGTexture;

class FColorCorrectRegionsSceneViewExtension : public FWorldSceneViewExtension
{
public:
	FColorCorrectRegionsSceneViewExtension(const FAutoRegister& AutoRegister, UWorld* InWorld, UColorCorrectRegionsSubsystem* InWorldSubsystem);
	
	//~ Begin FSceneViewExtensionBase Interface
	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override {};
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override {};
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual void PrePostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessingInputs& Inputs) override;
	//~ End FSceneViewExtensionBase Interface
public:
	// Called when owning subsystem needs to release this extension.
	void Invalidate();
private:
	UColorCorrectRegionsSubsystem* WorldSubsystem;
};
	