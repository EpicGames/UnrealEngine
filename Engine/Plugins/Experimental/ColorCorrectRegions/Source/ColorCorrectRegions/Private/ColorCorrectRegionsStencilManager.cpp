// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorCorrectRegionsStencilManager.h"
#include "ColorCorrectRegionsModule.h"
#include "Engine/GameEngine.h"
#include "Misc/MessageDialog.h"
#include "Components/PrimitiveComponent.h"
#include "EngineUtils.h"
#include "Engine/World.h"

#if WITH_EDITOR
#include "Editor.h"
#include "ScopedTransaction.h"
#endif


static TAutoConsoleVariable<int32> CVarStencilIdRangeMin(
	TEXT("r.CCR.StencilIdRangeMin"),
	127,
	TEXT("The lower boundary of stencil Ids that are reserved by CCR.\n\
		By default this value is set to 127. Must be within the range of [0, 255]."));

static TAutoConsoleVariable<int32> CVarStencilIdRangeMax(
	TEXT("r.CCR.StencilIdRangeMax"),
	255,
	TEXT("The upper boundary of stencil Ids that are reserved by CCR. \n\
		By default this value is set to 255. Must be within the range of [0, 255].\n"));

static TAutoConsoleVariable<bool> CVarEnablePopupStencilIdChangedOnActors(
	TEXT("r.CCR.EnablePopupStencilIdChangedOnActors"),
	false,
	TEXT("Allows CCRs to notify users of change in stencil Ids on actors modified by CCR. \n\
		By default no notification is displayed and CCR assumes that users changed stencil numbers intentionally.\n"));

#define LOCTEXT_NAMESPACE "CCR"

namespace
{
	/*
	* returns true if Actor is being used by another CCR and is currently sharing the same stencil number with other actors assigned to that CCR.
	* This means that we need to assign a new stencil number to this actor across all CCRs that use it.
	*
	* Also returns OutStencilDataToUse which is shared information about this actor between different CCRs. If OutStencilDataToUse is invalid
	* that means that there are no other CCRs that stencil this actor.
	*/
	static bool FindIfActorIsUsedByCCR(TSoftObjectPtr<AActor> ActorToSearchFor, const TArray<AActor*>& CCRsToSearchIn, TSharedPtr<FStencilData>& OutStencilDataToUse)
	{
		bool bAssignNewStencilId = false;
		OutStencilDataToUse = nullptr;

		for (const AActor* ActorCCR : CCRsToSearchIn)
		{
			const AColorCorrectRegion* CCR = Cast<AColorCorrectRegion>(ActorCCR);
			if (CCR->PerAffectedActorStencilData.Contains(ActorToSearchFor))
			{
				OutStencilDataToUse = CCR->PerAffectedActorStencilData[ActorToSearchFor];
				if (CCR->PerAffectedActorStencilData.Num() > 1)
				{
					TSharedPtr<FStencilData> TempStencilData = CCR->PerAffectedActorStencilData[ActorToSearchFor];
					for (const TPair<TSoftObjectPtr<AActor>, TSharedPtr<FStencilData>>& StencilDataPair : CCR->PerAffectedActorStencilData)
					{
						if (StencilDataPair.Key != ActorToSearchFor && StencilDataPair.Value->AssignedStencil == TempStencilData->AssignedStencil)
						{
							bAssignNewStencilId = true;
							return bAssignNewStencilId;
						}
					}
				}
			}
		}
		return bAssignNewStencilId;
	}

	/*
	* @param bIgnoreUserNotification	In cases when another CCR is already using this actor, we don't want to notify the user. Another example is when we reload the scene,
	* when all we need to do is reassign stencil ids.
	* @param bSoftAssign			In some cases we don't want to assign stencil ids to actors (for example when we reload the scene).
	*/
	static void AssignStencilIdToActor(AColorCorrectRegion* InRegion, bool bIgnoreUserNotification, bool bSoftAssign, TSoftObjectPtr<AActor> ActorToAssignStencilTo, TSharedPtr<FStencilData> StencilData)
	{
		TArray<UPrimitiveComponent*> PrimitiveComponents;
		ActorToAssignStencilTo->GetComponents<UPrimitiveComponent>(PrimitiveComponents);
#if WITH_EDITOR
		if (!bSoftAssign)
		{
			ActorToAssignStencilTo->Modify();
		}
#endif
		uint8 StencilIdMin = FMath::Min(CVarStencilIdRangeMin.GetValueOnAnyThread(), CVarStencilIdRangeMax.GetValueOnAnyThread());
		uint8 StencilIdMax = FMath::Max(CVarStencilIdRangeMin.GetValueOnAnyThread(), CVarStencilIdRangeMax.GetValueOnAnyThread());
		for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
		{
			// If the stencil id is already assigned and not by another CCR then we need to notify the user.
			if ((PrimitiveComponent->bRenderCustomDepth && PrimitiveComponent->CustomDepthStencilValue < StencilIdMin))
			{
				EAppReturnType::Type Answer = EAppReturnType::Yes;
				if (!bIgnoreUserNotification)
				{
					auto Text = FString::Printf(TEXT("The actor \'%s\' already has Custom Depth Stencil Id assigned that is out of range.\
													\nWould you like for Color Correct Regions to manage Stencil Id?\
													\nThe value will be reassigned to within managed range [%d %d]."), *ActorToAssignStencilTo->GetName(), StencilIdMin, StencilIdMax);
					Answer = FMessageDialog::Open(EAppMsgType::YesNo, FText::FromString(Text));

				}
				if (Answer == EAppReturnType::No)
				{
					StencilData->AssignedStencil = PrimitiveComponent->CustomDepthStencilValue;
					InRegion->PerAffectedActorStencilData.Add(ActorToAssignStencilTo, StencilData);
					return;
				}
			}
			if (!bSoftAssign)
			{
#if WITH_EDITOR
				PrimitiveComponent->Modify();
#endif
				PrimitiveComponent->SetRenderCustomDepth(true);
				PrimitiveComponent->CustomDepthStencilValue = static_cast<int32>(StencilData->AssignedStencil);
				PrimitiveComponent->MarkRenderStateDirty();
			}
			else
			{
				StencilData->AssignedStencil = PrimitiveComponent->CustomDepthStencilValue;
			}
		}
		InRegion->PerAffectedActorStencilData.Add(ActorToAssignStencilTo, StencilData);
	}

	/*
	* Handles the cleanup for the provided actor.
	*/
	static void ClearStencilIdFromActor(TSoftObjectPtr<AActor> ActorToAssignStencilTo)
	{
		uint8 StencilIdMin = FMath::Min(CVarStencilIdRangeMin.GetValueOnAnyThread(), CVarStencilIdRangeMax.GetValueOnAnyThread());
		if (ActorToAssignStencilTo.IsValid())
		{
			TArray<UPrimitiveComponent*> PrimitiveComponents;
			ActorToAssignStencilTo->GetComponents<UPrimitiveComponent>(PrimitiveComponents);

			bool bNeedsDirtying = false;
			TArray<UPrimitiveComponent*> PrimitiveComponentsToDirty;

			for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
			{
				// If the stencil id is within our range then it is managed by us, so we should reset this to default.
				if (PrimitiveComponent->bRenderCustomDepth && PrimitiveComponent->CustomDepthStencilValue >= StencilIdMin)
				{
					bNeedsDirtying = true;
					PrimitiveComponentsToDirty.Add(PrimitiveComponent);
				}
			}

			if (bNeedsDirtying
#if WITH_EDITOR
				&& GEditor
#endif
				)
			{
#if WITH_EDITOR
				const FScopedTransaction Transaction(LOCTEXT("PerActorCCActorCleaned", "Per actor CC Actor Stencil Id Cleared."));
				ActorToAssignStencilTo->Modify();
#endif

				for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponentsToDirty)
				{
#if WITH_EDITOR
					PrimitiveComponent->Modify();
#endif
					PrimitiveComponent->SetRenderCustomDepth(false);
					PrimitiveComponent->CustomDepthStencilValue = 0;
					PrimitiveComponent->MarkRenderStateDirty();
				}
			}
		}
	}

	/*
	* Simple checking function that returns true if the provided actor is used by another CCR.
	*/
	static bool IsActorAssignedToOtherCCR(AColorCorrectRegion* CurrentCCR, TSoftObjectPtr<AActor> ActorToSearchFor, const TArray<AActor*>& CCRsToSearchIn)
	{
		bool bAssignNewStencilId = false;

		for (const AActor* ActorCCR : CCRsToSearchIn)
		{
			const AColorCorrectRegion* CCR = Cast<AColorCorrectRegion>(ActorCCR);
			if (CurrentCCR == CCR)
			{
				continue;
			}
			if (CCR->PerAffectedActorStencilData.Contains(ActorToSearchFor))
			{
				return true;
			}
		}
		return false;
	}
}


void FColorCorrectRegionsStencilManager::AssignStencilNumberToActorForSelectedRegion(UWorld* CurrentWorld, AColorCorrectRegion* Region, TSoftObjectPtr<AActor> ActorToAssignStencilTo, bool bIgnoreUserNotifications, bool bSoftAssign)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("CCR.StencilManager.AssignStencilNumberToActorForSelectedRegion %s"), *Region->GetName()));
	verify(ActorToAssignStencilTo.IsValid());
	
	TArray<FStencilData> StencilDatas;

	ULevel* CurrentLevel = CurrentWorld->GetCurrentLevel();
	if (Region->PerAffectedActorStencilData.Contains(ActorToAssignStencilTo) && Region->PerAffectedActorStencilData[ActorToAssignStencilTo]->AssignedStencil != 0)
	{
		// This actor is already assigned to the current CCR. No need to do anything.
		return;
	}

	// This array is an optimization that avoids additional costs of going over all actors in the scene and searching for CCRs.
	TArray<AActor*> AllCCRsInCurrentLevel;

	TSharedPtr<FStencilData> StencilDataToUse;
	TSortedMap<uint32, TSharedPtr<FStencilData>> AllUsedIds;

	// Gather all unique stencil ids that are currently in use.
	for (TActorIterator<AColorCorrectRegion> It(CurrentWorld); It; ++It)
	{
		AColorCorrectRegion* CCR = *It;
		AllCCRsInCurrentLevel.Add(CCR);
		for (const TPair<TSoftObjectPtr<AActor>, TSharedPtr<FStencilData>>& StencilDataPair : CCR->PerAffectedActorStencilData)
		{
			AllUsedIds.Add(StencilDataPair.Value->AssignedStencil, StencilDataPair.Value);
		}
	}

	// Find if stencil number is already assigned to the current actor by another CCR.
	bool bAssignNewStencilId = FindIfActorIsUsedByCCR(ActorToAssignStencilTo, AllCCRsInCurrentLevel, StencilDataToUse);

	bool bUsedByAnotherCCR = StencilDataToUse.IsValid();

	uint8 StencilIdMin = FMath::Min(CVarStencilIdRangeMin.GetValueOnAnyThread(), CVarStencilIdRangeMax.GetValueOnAnyThread());
	uint8 StencilIdMax = FMath::Max(CVarStencilIdRangeMin.GetValueOnAnyThread(), CVarStencilIdRangeMax.GetValueOnAnyThread());

	/*
	* If the actor is not used by any other CCR, it means that we need to see if we can group up stencil id
	* for that actor with another, that already belongs to the current CCR,
	*
	* Find Actors that are assigned to the current CCR and make sure these are unused by any other CCR before
	* using that stencil ID
	*/
	if (!StencilDataToUse.IsValid())
	{
		StencilDataToUse = MakeShared<FStencilData>();		
		bAssignNewStencilId = true;

		if (Region->PerAffectedActorStencilData.Num() > 0)
		{
			for (const TPair<TSoftObjectPtr<AActor>, TSharedPtr<FStencilData>>& ActorDataPair : Region->PerAffectedActorStencilData)
			{
				if (!IsActorAssignedToOtherCCR(Region, ActorDataPair.Key, AllCCRsInCurrentLevel))
				{
					if (ActorDataPair.Value->AssignedStencil >= StencilIdMin
						&& ActorDataPair.Value->AssignedStencil <= StencilIdMax)
					{
						// we reuse stencil Id of another actor which is assigned to the same CCR. 
						bAssignNewStencilId = false;
						StencilDataToUse->AssignedStencil = ActorDataPair.Value->AssignedStencil;
						break;
					}
				}
			}
		}
	}

	/*
	* Stencil Id needs doesn't need to be assigned, for either of the following reasons:
	* - We reuse stencil Id from another actor that is assigned to the same CCR (Group up ids)
	* - This actor is used by another CCR and that CCR doesn't have any other actors, so we can reuse the same id.
	*/
	if (!bAssignNewStencilId)
	{
		AssignStencilIdToActor(Region, bUsedByAnotherCCR || bIgnoreUserNotifications, bSoftAssign, ActorToAssignStencilTo, StencilDataToUse);
		return;
	}

	if (bAssignNewStencilId)
	{
		uint8 NextAvailableStencilId = StencilIdMin;

		while (NextAvailableStencilId <= StencilIdMax)
		{
			if (!AllUsedIds.Contains(NextAvailableStencilId))
			{
				break;
			}
			NextAvailableStencilId++;
		}
		if (NextAvailableStencilId > StencilIdMax)
		{
			UE_LOGF(ColorCorrectRegions, Error, "Run out of Custom Depth Stencil ID values to be used. The maximum stencil ID is assigned. This may yeild unexpected results. ");
			NextAvailableStencilId = StencilIdMax;
		}
		StencilDataToUse->AssignedStencil = NextAvailableStencilId;
		AssignStencilIdToActor(Region, bUsedByAnotherCCR || bIgnoreUserNotifications, bSoftAssign, ActorToAssignStencilTo, StencilDataToUse);
	}
}

void FColorCorrectRegionsStencilManager::RemoveStencilNumberForSelectedRegion(UWorld* CurrentWorld, AColorCorrectRegion* Region)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("CCR.StencilManager.RemoveStencilNumberForSelectedRegion %s"), *Region->GetName()));
	TSet<TSoftObjectPtr<AActor>> ActorsToCleanup;

	// Accumulate all CCRs that were removed.
	for (const TPair<TSoftObjectPtr<AActor>, TSharedPtr<FStencilData>>& ActorDataPair : Region->PerAffectedActorStencilData)
	{
		if (!Region->AffectedActors.Contains(ActorDataPair.Key))
		{
			ActorsToCleanup.Add(ActorDataPair.Key);
		}
	}

	ULevel* CurrentLevel = CurrentWorld->GetCurrentLevel();
	TArray<AActor*> AllCCRsInCurrentLevel = ObjectPtrDecay(CurrentLevel->Actors).FilterByPredicate([](const AActor* Actor) { return Cast<AColorCorrectRegion>(Actor) != nullptr; });

	// For each actor that was removed, remove Ids if it is unused by any other CCR, or otherwise assign a new id.
	for (TSoftObjectPtr<AActor> ActorToRemove : ActorsToCleanup)
	{
		Region->PerAffectedActorStencilData.Remove(ActorToRemove);

		bool bUsedByAnotherActor = false;
		for (AActor* ActorCCR : AllCCRsInCurrentLevel)
		{
			AColorCorrectRegion* OtherCCR = Cast<AColorCorrectRegion>(ActorCCR);

			if (OtherCCR->PerAffectedActorStencilData.Contains(ActorToRemove))
			{
				OtherCCR->PerAffectedActorStencilData.Remove(ActorToRemove);
				AssignStencilNumberToActorForSelectedRegion(CurrentWorld, OtherCCR, ActorToRemove, true, false);
				bUsedByAnotherActor = true;
				break;
			}
		}

		// This actor is not used by any aother actor. Remove it.
		if (!bUsedByAnotherActor)
		{
			ClearStencilIdFromActor(ActorToRemove);
		}
	}
}

void FColorCorrectRegionsStencilManager::OnCCRRemoved(UWorld* CurrentWorld, AColorCorrectRegion* Region)
{
	ULevel* CurrentLevel = CurrentWorld->GetCurrentLevel();
	TArray<AActor*> AllCCRsInCurrentLevel = ObjectPtrDecay(CurrentLevel->Actors).FilterByPredicate([](const AActor* Actor) { return Cast<AColorCorrectRegion>(Actor) != nullptr; });

	// For each actor that was removed, remove Ids if it is unused by any other CCR, or otherwise assign a new id.
	for (const TPair<TSoftObjectPtr<AActor>, TSharedPtr<FStencilData>>& ActorDataPair : Region->PerAffectedActorStencilData)
	{
		bool bUsedByAnotherActor = false;
		for (AActor* ActorCCR : AllCCRsInCurrentLevel)
		{
			AColorCorrectRegion* OtherCCR = Cast<AColorCorrectRegion>(ActorCCR);
			if (OtherCCR == Region)
			{
				continue;
			}

			if (OtherCCR->PerAffectedActorStencilData.Contains(ActorDataPair.Key))
			{
				OtherCCR->PerAffectedActorStencilData.Remove(ActorDataPair.Key);
				AssignStencilNumberToActorForSelectedRegion(CurrentWorld, OtherCCR, ActorDataPair.Key, true, false);
				bUsedByAnotherActor = true;
				break;
			}
		}

		// This actor is not used by any aother actor. Remove it.
		if (!bUsedByAnotherActor)
		{
			ClearStencilIdFromActor(ActorDataPair.Key);
		}
	}
	Region->PerAffectedActorStencilData.Empty();
}

void FColorCorrectRegionsStencilManager::ClearInvalidActorsForSelectedRegion(AColorCorrectRegion* Region)
{
	TArray<TSoftObjectPtr<AActor>> ActorsToCleanup;
	for (TSoftObjectPtr<AActor> Actor : Region->AffectedActors)
	{
		if (!Actor.IsValid())
		{
			ActorsToCleanup.Add(Actor);
		}
	}
	for (TSoftObjectPtr<AActor> Actor : ActorsToCleanup)
	{
		Region->AffectedActors.Remove(Actor);
	}
}

void FColorCorrectRegionsStencilManager::AssignStencilIdsToAllActorsForCCR(UWorld* CurrentWorld, AColorCorrectRegion* Region, bool bIgnoreUserNotifications, bool bSoftAssign)
{
	if (Region->AffectedActors.Num() > 0)
	{
		for (TSoftObjectPtr<AActor> Actor : Region->AffectedActors)
		{
			if (Actor.IsValid())
			{
				FColorCorrectRegionsStencilManager::AssignStencilNumberToActorForSelectedRegion(CurrentWorld, Region, Actor, bIgnoreUserNotifications, bSoftAssign);
			}
		}
	}
}

void FColorCorrectRegionsStencilManager::CleanActor(AActor* Actor)
{
	ClearStencilIdFromActor(Actor);
}

void FColorCorrectRegionsStencilManager::CheckAssignedActorsValidity(AColorCorrectRegion* Region)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("CCR.StencilManager.CheckAssignedActorsValidity %s"), *Region->GetName()));

	if (Region->PerAffectedActorStencilData.Num() > 0)
	{
		TMap<TSoftObjectPtr<AActor>, TSharedPtr<FStencilData>> ActorsToUpdate;
		for (const TPair<TSoftObjectPtr<AActor>, TSharedPtr<FStencilData>>& StencilDataPair : Region->PerAffectedActorStencilData)
		{
			TSoftObjectPtr<AActor> Actor = StencilDataPair.Key;
			if (Actor.IsValid())
			{
				TArray<UPrimitiveComponent*> PrimitiveComponents;

				Actor->GetComponents<UPrimitiveComponent>(PrimitiveComponents);
				if (AColorCorrectRegion* CCRCast = Cast<AColorCorrectRegion>(Actor.Get()))
				{
					continue;
				}
				for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
				{
					if (PrimitiveComponent->CustomDepthStencilValue != StencilDataPair.Value->AssignedStencil || !PrimitiveComponent->bRenderCustomDepth)
					{
						auto Text = FString::Printf(TEXT("Custom Stencil Id for Actor: '%s', Component: '%s' is managed by Color Correct Region '%s' and has been modified manually. Would you like to change the id back?"), *Actor->GetName(), *PrimitiveComponent->GetName(), *Region->GetName());
						EAppReturnType::Type Answer = EAppReturnType::No;
						if (CVarEnablePopupStencilIdChangedOnActors.GetValueOnAnyThread())
						{
							Answer = FMessageDialog::Open(EAppMsgType::YesNo, FText::FromString(Text));
						}
						if (Answer == EAppReturnType::No)
						{
							// Update our stored value and then update other components with the new stencil Id value.
							StencilDataPair.Value->AssignedStencil = PrimitiveComponent->CustomDepthStencilValue;
						}
						ActorsToUpdate.Add(StencilDataPair.Key, StencilDataPair.Value);
						break;
					}
				}
			}
		}

		for (const TPair<TSoftObjectPtr<AActor>, TSharedPtr<FStencilData>>& ActorToUpdate : ActorsToUpdate)
		{
			Region->PerAffectedActorStencilData.Remove(ActorToUpdate.Key);

			// Assign stencil values without notifying the user.
			AssignStencilIdToActor(Region, true, false, ActorToUpdate.Key, ActorToUpdate.Value);

		}
	}
}

#undef LOCTEXT_NAMESPACE