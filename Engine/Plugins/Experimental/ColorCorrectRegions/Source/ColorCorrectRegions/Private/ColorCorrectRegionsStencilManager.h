// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ColorCorrectRegion.h"
#include "GameFramework/Actor.h"


class FColorCorrectRegionsStencilManager
{
public:
		static void AssignStencilNumberToActorForSelectedRegion(UWorld* CurrentWorld, AColorCorrectRegion* Region, TSoftObjectPtr<AActor> ActorToAssignStencilTo, bool bIgnoreUserNotifications, bool bSoftAssign);
		static void RemoveStencilNumberForSelectedRegion(UWorld* CurrentWorld, AColorCorrectRegion* Region);
		static void OnCCRRemoved(UWorld* CurrentWorld, AColorCorrectRegion* Region);
		static void ClearInvalidActorsForSelectedRegion(AColorCorrectRegion* Region);
		static void AssignStencilIdsToAllActorsForCCR(UWorld* CurrentWorld, AColorCorrectRegion* Region, bool bIgnoreUserNotifications, bool bSoftAssign);
		static void CleanActor(AActor* Actor);

		/** Runs over all actors assigned to the selected Region and makes sure that Stencil Ids match the ones assigned to it. */
		static void CheckAssignedActorsValidity(AColorCorrectRegion* Region);
};