// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Subsystems/EngineSubsystem.h"
#include "Engine/EngineBaseTypes.h"
#include "ColorCorrectRegion.h"
#include "ColorCorrectRegionsSceneViewExtension.h"

#if WITH_EDITOR
#include "EditorUndoClient.h"
#endif

#if WITH_EDITOR
#include "MultiUser/ColorCorrectRegionsMultiUserManager.h"
#endif

#include "ColorCorrectRegionsSubsystem.generated.h"

/**
 * World Subsystem responsible for managing AColorCorrectRegion classes in level.
 * This subsystem handles:
 *		Level Loaded, Undo/Redo, Added to level, Removed from level events.
 * Unfortunately AActor class itself is not aware of when it is added/removed, Undo/Redo etc in the level.
 * 
 * This is the only way (that we found) that was handling all region aggregation cases in more or less efficient way.
 *		Covered cases: Region added to a level, deleted from level, level loaded, undo, redo, level closed, editor closed:
 *		World subsystem keeps track of all Regions in a level via three events OnLevelActorAdded and OnLevelActorDeleted.
 *		Actor classes are unaware of when they are added/deleted/undo/redo etc in the level, therefore this is the best place to manage this.
 * Alternative strategies (All tested):
 *		World's AddOnActorSpawnedHandler. Flawed. Invoked in some cases we don't need, but does not get called during UNDO/REDO
 *		AActor's PostSpawnInitialize, PostActorCreated  and OnConstruction are also flawed.
 *		AActor does not have an internal event for when its deleted (EndPlay is the closest we have).
 */
UCLASS()
class UColorCorrectRegionsSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()
public:

	// Subsystem Init/Deinit
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

	virtual bool IsTickableInEditor() const { return true; }
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UColorCorrectRegionsSubsystem, STATGROUP_Tickables); }

public:

	/** A callback for CC Region creation. */
	void OnActorSpawned(AActor* InActor);

	/** A callback for CC Region deletion. */
	void OnActorDeleted(AActor* InActor, bool bClearStencilIdValues);

	/** Called when duplication process is started in the level. */
	void OnDuplicateActorsBegin() { bDuplicationStarted = true; };

	/** Called when duplication process is ended in the level. */
	void OnDuplicateActorsEnd();

	/** Handles Stencil Ids for the selected CCR and corresponding actor. */
	void AssignStencilIdsToPerActorCC(AColorCorrectRegion* Region, bool bIgnoreUserNotificaion = false, bool bSoftAssign = false);
	
	/** Handles removal of Stencil Ids for the selected CCR. */
	void ClearStencilIdsToPerActorCC(AColorCorrectRegion* Region);	
	
	/** Handles cases when stencil Id has been changed from outside by the user manually. */
	void CheckAssignedActorsValidity(AColorCorrectRegion* Region);

	/**
	 * Notifies connected multi-user clients that per-actor CC stencil assignment changed for
	 * the given region. Called after local stencil operations complete. No-op when no Concert
	 * session is active or in non-editor builds.
	 */
	void NotifyPerActorCCChanged(AColorCorrectRegion* Region);

private:

	/** Repopulates array of region actors. */
	void RefreshRegions();

	/**
	* Copy states required for rendering to be consumed by Scene view extension to render all active 
	* CCRs and CCWs.
	*/
	void TransferStates();

private:

	/** Stores pointers to ColorCorrectRegion Actors that use priority for sorting. */
	TArray<TWeakObjectPtr<AColorCorrectRegion>> RegionsPriorityBased;
	/** Stores pointers to ColorCorrectRegion Actors that are based on distance from camera. */
	TArray<TWeakObjectPtr<AColorCorrectRegion>> RegionsDistanceBased;


	/** Proxies to be used exclusively on render thread. Copies of the state of CC Actors sorted by priority. */
	TArray<FColorCorrectRenderProxyPtr> ProxiesPriorityBased;
	/** Proxies to be used exclusively on render thread. Copies of the state of CC Actors sorted by distance. */
	TArray<FColorCorrectRenderProxyPtr> ProxiesDistanceBased;

	TSharedPtr< class FColorCorrectRegionsSceneViewExtension, ESPMode::ThreadSafe > PostProcessSceneViewExtension;

	/** This is to handle actor duplication for Per Actor CC. */
	bool bDuplicationStarted = false;
	TArray<AActor*> DuplicatedActors;

	// This is for optimization purposes that would let us check assigned actors component's stencil ids every once in a while.
	float TimeSinceLastValidityCheck = 0;

#if WITH_EDITOR
	/** Sends Concert custom events when Per Actor CC assignment changes. Null when Concert is unavailable. */
	TUniquePtr<FCCRMultiUserManager> MultiUserManager;
#endif

public:
	friend class FColorCorrectRegionsSceneViewExtension;
};