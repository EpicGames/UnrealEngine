// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorCorrectWindow.h"
#include "Components/BillboardComponent.h"
#include "CoreMinimal.h"
#include "Materials/Material.h"
#include "Engine/CollisionProfile.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Misc/EnumRange.h"
#include "UObject/ConstructorHelpers.h"

ENUM_RANGE_BY_COUNT(EColorCorrectWindowType, EColorCorrectWindowType::MAX)

AColorCorrectionWindow::AColorCorrectionWindow(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, WindowType(EColorCorrectWindowType::Square)
{
	PositionalParams.DistanceFromCenter = 300.f;
	PositionalParams.Longitude = 0.f;
	PositionalParams.Latitude = 30.f;
	PositionalParams.Spin = 0.f;
	PositionalParams.Pitch = 0.f;
	PositionalParams.Yaw = 0.5;
	PositionalParams.RadialOffset = -0.5f;
	PositionalParams.Scale = FVector2D(1.f);
	
	UMaterial* Material = LoadObject<UMaterial>(NULL, TEXT("/ColorCorrectRegions/Materials/M_ColorCorrectRegionTransparentPreview.M_ColorCorrectRegionTransparentPreview"), NULL, LOAD_None, NULL);
	const TArray<UStaticMesh*> StaticMeshes =
	{
		Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, TEXT("/Engine/BasicShapes/Plane"))),
		Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, TEXT("/Engine/BasicShapes/Cylinder")))
	};
	for (EColorCorrectWindowType CCWType : TEnumRange<EColorCorrectWindowType>())
	{
		UStaticMeshComponent* MeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(*UEnum::GetValueAsString(CCWType));
		MeshComponents.Add(MeshComponent);
		MeshComponent->SetupAttachment(RootComponent);
		MeshComponent->SetStaticMesh(StaticMeshes[static_cast<uint8>(CCWType)]);
		MeshComponent->SetWorldScale3D(FVector(1., 1., 0.001));
		MeshComponent->SetMaterial(0, Material);
		MeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		MeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		MeshComponent->CastShadow = false;
		MeshComponent->SetHiddenInGame(true);
	}

	ChangeShapeVisibilityForActorType();

#if WITH_METADATA
	CreateIcon();
#endif
}

AColorCorrectionWindow::~AColorCorrectionWindow()
{
}

#if WITH_EDITOR
void AColorCorrectionWindow::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(AColorCorrectionWindow, WindowType))
	{
		ChangeShapeVisibilityForActorType();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void AColorCorrectionWindow::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);
}

FName AColorCorrectionWindow::GetCustomIconName() const
{
	return TEXT("CCW.OutlinerThumbnail");
}

#endif //WITH_EDITOR


#if WITH_METADATA
void AColorCorrectionWindow::CreateIcon()
{
	// Create billboard component
	if (GIsEditor && !IsRunningCommandlet())
	{
		// Structure to hold one-time initialization

		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTextureObject;
			FName ID_ColorCorrectRegion;
			FText NAME_ColorCorrectRegion;

			FConstructorStatics()
				: SpriteTextureObject(TEXT("/ColorCorrectRegions/Icons/S_ColorCorrectWindowIcon"))
				, ID_ColorCorrectRegion(TEXT("Color Correct Window"))
				, NAME_ColorCorrectRegion(NSLOCTEXT("SpriteCategory", "ColorCorrectWindow", "Color Correct Window"))
			{
			}
		};

		static FConstructorStatics ConstructorStatics;

		SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Color Correct Window Icon"));

		if (SpriteComponent)
		{
			SpriteComponent->Sprite = ConstructorStatics.SpriteTextureObject.Get();
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_ColorCorrectRegion;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_ColorCorrectRegion;
			SpriteComponent->SetIsVisualizationComponent(true);
			SpriteComponent->SetRelativeLocationAndRotation(FVector::ZeroVector, FRotator::ZeroRotator);
			SpriteComponent->SetMobility(EComponentMobility::Movable);
			SpriteComponent->bHiddenInGame = true;
			SpriteComponent->bIsScreenSizeScaled = true;

			SpriteComponent->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepRelativeTransform);
		}
	}

}
#endif 

void AColorCorrectionWindow::ChangeShapeVisibilityForActorType()
{
	ChangeShapeVisibilityForActorTypeInternal<EColorCorrectWindowType>(WindowType);
}

#if WITH_EDITOR
void AColorCorrectionWindow::FixMeshComponentReferences()
{
	FixMeshComponentReferencesInternal<EColorCorrectWindowType>(WindowType);
}
#endif

ADEPRECATED_ColorCorrectWindow::ADEPRECATED_ColorCorrectWindow(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}
