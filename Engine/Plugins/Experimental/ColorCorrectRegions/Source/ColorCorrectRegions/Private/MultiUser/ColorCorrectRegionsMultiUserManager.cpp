// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "MultiUser/ColorCorrectRegionsMultiUserManager.h"
#include "MultiUser/ColorCorrectRegionsSyncData.h"
#include "ColorCorrectRegion.h"
#include "ColorCorrectRegionsSubsystem.h"

#include "ConcertTransportMessages.h"
#include "IConcertClient.h"
#include "IConcertSession.h"
#include "IConcertSessionHandler.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"

#include "Engine/World.h"
#include "EngineUtils.h"


FCCRMultiUserManager::FCCRMultiUserManager(UColorCorrectRegionsSubsystem* InSubsystem)
	: WeakSubsystem(InSubsystem)
{
	IConcertSyncClientModule* ConcertModule = FModuleManager::GetModulePtr<IConcertSyncClientModule>("ConcertSyncClient");
	if (!ConcertModule)
	{
		return;
	}

	TSharedPtr<IConcertSyncClient> ConcertSyncClient = ConcertModule->GetClient(TEXT("MultiUser"));
	if (!ConcertSyncClient)
	{
		return;
	}

	IConcertClientRef Client = ConcertSyncClient->GetConcertClient();
	ConcertClient = Client;
	SessionStartupHandle = Client->OnSessionStartup().AddRaw(this, &FCCRMultiUserManager::OnSessionStartup);
	SessionShutdownHandle = Client->OnSessionShutdown().AddRaw(this, &FCCRMultiUserManager::OnSessionShutdown);
	SessionConnectionChangedHandle = Client->OnSessionConnectionChanged().AddRaw(this, &FCCRMultiUserManager::OnSessionConnectionChanged);

	// If a session is already active when the subsystem initialises (e.g. level loaded while
	// already in multi-user), register the event handler immediately.
	if (TSharedPtr<IConcertClientSession> CurrentSession = Client->GetCurrentSession())
	{
		OnSessionStartup(CurrentSession.ToSharedRef());
	}
}

FCCRMultiUserManager::~FCCRMultiUserManager()
{
	if (TSharedPtr<IConcertClient> Client = ConcertClient.Pin())
	{
		Client->OnSessionStartup().Remove(SessionStartupHandle);
		Client->OnSessionShutdown().Remove(SessionShutdownHandle);
		Client->OnSessionConnectionChanged().Remove(SessionConnectionChangedHandle);
	}

	if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
	{
		Session->UnregisterCustomEventHandler<FConcertCCRPerActorAssignmentEvent>(this);
	}
}

void FCCRMultiUserManager::OnSessionStartup(TSharedRef<IConcertClientSession> Session)
{
	if (TSharedPtr<IConcertClientSession> PreviousSession = WeakSession.Pin())
	{
		PreviousSession->UnregisterCustomEventHandler<FConcertCCRPerActorAssignmentEvent>(this);
	}
	WeakSession = Session;
	Session->RegisterCustomEventHandler<FConcertCCRPerActorAssignmentEvent>(
		this, &FCCRMultiUserManager::HandlePerActorAssignmentEvent);
}

void FCCRMultiUserManager::OnSessionShutdown(TSharedRef<IConcertClientSession> Session)
{
	Session->UnregisterCustomEventHandler<FConcertCCRPerActorAssignmentEvent>(this);
	WeakSession.Reset();
}

void FCCRMultiUserManager::OnSessionConnectionChanged(IConcertClientSession& Session, EConcertConnectionStatus ConnectionStatus)
{
	if (ConnectionStatus == EConcertConnectionStatus::Connected)
	{
		if (TSharedPtr<IConcertClient> Client = ConcertClient.Pin())
		{
			if (TSharedPtr<IConcertClientSession> CurrentSession = Client->GetCurrentSession())
			{
				// Delegate to OnSessionStartup so handler registration and WeakSession
				// are set from the same TSharedRef, avoiding a mismatch with Session&.
				OnSessionStartup(CurrentSession.ToSharedRef());
			}
		}
	}
	else if (ConnectionStatus == EConcertConnectionStatus::Disconnecting)
	{
		Session.UnregisterCustomEventHandler<FConcertCCRPerActorAssignmentEvent>(this);
		WeakSession.Reset();
	}
}

void FCCRMultiUserManager::SendPerActorAssignmentEvent(AColorCorrectRegion* Region)
{
	TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
	if (!Session || !Region)
	{
		return;
	}

	FConcertCCRPerActorAssignmentEvent Event;
	Event.CCRActorPath = FSoftObjectPath(Region);

	// Snapshot the full AffectedActors list so the receiver doesn't depend on Concert's
	// property transaction having arrived first.
	for (const TSoftObjectPtr<AActor>& Actor : Region->AffectedActors)
	{
		Event.AffectedActors.Add(Actor.ToSoftObjectPath());
	}

	Session->SendCustomEvent(Event, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
}

void FCCRMultiUserManager::HandlePerActorAssignmentEvent(
	const FConcertSessionContext& Context,
	const FConcertCCRPerActorAssignmentEvent& Event)
{
	// Ignore events that originated from this machine, or if the session is no longer valid.
	TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
	if (!Session || Context.SourceEndpointId == Session->GetSessionClientEndpointId())
	{
		return;
	}

	UColorCorrectRegionsSubsystem* Subsystem = WeakSubsystem.Get();
	if (!Subsystem)
	{
		return;
	}

	UWorld* World = Subsystem->GetWorld();
	if (!World)
	{
		return;
	}

	AColorCorrectRegion* Region = Cast<AColorCorrectRegion>(Event.CCRActorPath.ResolveObject());
	if (!Region || Region->GetWorld() != World)
	{
		return;
	}

	// Populate AffectedActors from the event so stencil assignment has the correct actor list
	// regardless of whether Concert's own property transaction has arrived yet.
	Region->AffectedActors.Empty();
	for (const FSoftObjectPath& Path : Event.AffectedActors)
	{
		Region->AffectedActors.Add(TSoftObjectPtr<AActor>(Path));
	}

	// Clear all stencil IDs for the region, then re-assign for the current AffectedActors list.
	Subsystem->ClearStencilIdsToPerActorCC(Region);
	Subsystem->AssignStencilIdsToPerActorCC(Region, /*bIgnoreUserNotification=*/true);
}


#endif // WITH_EDITOR
