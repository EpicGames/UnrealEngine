// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class AColorCorrectRegion;
class IConcertClient;
class IConcertClientSession;
class UColorCorrectRegionsSubsystem;
struct FConcertCCRPerActorAssignmentEvent;
struct FConcertSessionContext;
enum class EConcertConnectionStatus : uint8;

/**
 * Manages multi-user synchronization of per-actor CC stencil ID assignments.
 *
 * When the per-actor CC AffectedActors list changes on one machine, this manager sends a
 * FConcertCCRPerActorAssignmentEvent Concert custom event. Passive clients in the session
 * receive the event and re-apply stencil assignments immediately rather than waiting for
 * the subsystem tick to detect the missing PerAffectedActorStencilData.
 *
 * Owned by UColorCorrectRegionsSubsystem; lifetime is aligned with the subsystem.
 */
class FCCRMultiUserManager
{
public:
	explicit FCCRMultiUserManager(UColorCorrectRegionsSubsystem* InSubsystem);
	~FCCRMultiUserManager();

	/**
	 * Sends a Concert custom event to all connected clients indicating that per-actor CC
	 * stencil assignment changed for the given region. Should be called after local stencil
	 * assignment is complete. Does nothing when no Concert session is active.
	 */
	void SendPerActorAssignmentEvent(AColorCorrectRegion* Region);

private:
	void OnSessionStartup(TSharedRef<IConcertClientSession> Session);
	void OnSessionShutdown(TSharedRef<IConcertClientSession> Session);
	void OnSessionConnectionChanged(IConcertClientSession& Session, EConcertConnectionStatus ConnectionStatus);

	/** Called when a remote client sends a FConcertCCRPerActorAssignmentEvent. */
	void HandlePerActorAssignmentEvent(
		const FConcertSessionContext& Context,
		const FConcertCCRPerActorAssignmentEvent& Event);

	TWeakObjectPtr<UColorCorrectRegionsSubsystem> WeakSubsystem;
	TWeakPtr<IConcertClientSession> WeakSession;
	TWeakPtr<IConcertClient> ConcertClient;
	FDelegateHandle SessionStartupHandle;
	FDelegateHandle SessionShutdownHandle;
	FDelegateHandle SessionConnectionChangedHandle;
};

#endif // WITH_EDITOR
