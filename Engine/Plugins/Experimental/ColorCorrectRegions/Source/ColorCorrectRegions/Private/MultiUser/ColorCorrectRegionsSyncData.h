// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/SoftObjectPath.h"

#include "ColorCorrectRegionsSyncData.generated.h"

/**
 * Concert custom event sent when the per-actor CC AffectedActors list changes on a Color Correction Actor.
 *
 * The event is self-contained: it carries the full current AffectedActors state so the receiving
 * client can apply stencil assignments immediately without waiting for Concert's property transaction
 * to arrive and update AffectedActors first.
 */
USTRUCT()
struct FConcertCCRPerActorAssignmentEvent
{
	GENERATED_BODY()

	/** Soft path to the CCR actor whose per-actor CC assignment changed. */
	UPROPERTY()
	FSoftObjectPath CCRActorPath;

	/** Full AffectedActors list as it stands on the sending client after the change. */
	UPROPERTY()
	TArray<FSoftObjectPath> AffectedActors;
};

