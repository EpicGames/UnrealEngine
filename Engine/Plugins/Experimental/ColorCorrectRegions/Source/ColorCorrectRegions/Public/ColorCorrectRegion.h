// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Scene.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Components/StaticMeshComponent.h"

#include "StageActor/IDisplayClusterStageActor.h"

#include "ColorCorrectRegion.generated.h"


class UColorCorrectRegionsSubsystem;
class UBillboardComponent;


UENUM(BlueprintType)
enum class EColorCorrectRegionsType : uint8 
{
	Sphere		UMETA(DisplayName = "Sphere"),
	Box			UMETA(DisplayName = "Box"),
	Cylinder	UMETA(DisplayName = "Cylinder"),
	Cone		UMETA(DisplayName = "Cone"),
	MAX
};

UENUM(BlueprintType)
enum class EColorCorrectWindowType : uint8
{
	Square		UMETA(DisplayName = "Square"),
	Circle		UMETA(DisplayName = "Circle"),
	MAX
};

UENUM(BlueprintType)
enum class EColorCorrectRegionTemperatureType : uint8
{
	LegacyTemperature		UMETA(DisplayName = "Temperature (Legacy)"),
	WhiteBalance			UMETA(DisplayName = "White Balance"),
	ColorTemperature		UMETA(DisplayName = "Color Temperature"),
	MAX
};

UENUM(BlueprintType)
enum class EColorCorrectRegionStencilType : uint8
{
	ExcludeStencil			UMETA(DisplayName = "Exclude Selected Actors"),
	IncludeStencil			UMETA(DisplayName = "Affect Only Selected Actors"),
	MAX
};

class FStencilData
{
public:
	uint32 AssignedStencil = 0;
};

/** A state to store a copy of CCR properties to be used on render thread. */
struct FColorCorrectRenderProxy
{
	enum EProxyType
	{
		PriorityBased,
		DistanceBased
	};

	EProxyType ProxyType;

	union {
		EColorCorrectRegionsType Type;
		EColorCorrectWindowType WindowType;
	};

	/** CCR Properties */
	int32 Priority;
	float Intensity;
	float Inner;
	float Outer;
	float Falloff;
	bool Invert;
	EColorCorrectRegionTemperatureType TemperatureType;
	float Temperature;
	float Tint;
	FColorGradingSettings ColorGradingSettings;

	bool bEnablePerActorCC;
	/**
	* A list of actors that need to be excluded or included from color correction to be used on render thread.
	* This list is populated during engine tick and populated with Stencil Ids from Primitive Components of AffectedActors.
	* The main reason for this is accessing Primitive Components is not thread-safe and should be accessed on Game thread.
	*/
	TArray<uint32> StencilIds;
	EColorCorrectRegionStencilType PerActorColorCorrection;

	TWeakObjectPtr<UWorld> World;
	FVector BoxOrigin;
	FVector BoxExtent;

	FVector3f ActorLocation;
	FVector3f ActorRotation;
	FVector3f ActorScale;

	FPrimitiveComponentId FirstPrimitiveId;

	bool bIsActiveThisFrame;
};

typedef TSharedPtr<FColorCorrectRenderProxy, ESPMode::ThreadSafe> FColorCorrectRenderProxyPtr;

/**
 * An instance of Color Correction Region. Used to aggregate all active regions.
 * This actor is aggregated by ColorCorrectRegionsSubsystem on Tick. 
 * AActor class itself is not aware of when it is added/removed, Undo/Redo etc in the Editor. 
 * More information in ColorCorrectRegionsSubsytem.h
 */
UCLASS(Blueprintable, NotPlaceable, Abstract, HideCategories=(Actor, HLOD, Collision, Replication, Cooking, Input, DataLayers, Physics, Streaming, WorldPartition))
class COLORCORRECTREGIONS_API AColorCorrectRegion : public AActor, public IDisplayClusterStageActor
{
	GENERATED_UCLASS_BODY()
public:
	UPROPERTY()
	TArray<TObjectPtr<UStaticMeshComponent>> MeshComponents;

public:
	virtual ~AColorCorrectRegion() override;
	
	/** Region type. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Region")
	EColorCorrectRegionsType Type;

	/** 
	* Render priority/order. The higher the number the later region will be applied. 
	* A region with Priority 1 will be rendered before a region with Priority 10. 
	* This property is hidden if priority is determined by distance from the camera (When Window CCR is being used). 
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Region")
	int32 Priority;

	/** Color correction intensity. Clamped to 0-1 range. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Region", meta = (UIMin = 0.0, UIMax = 1.0))
	float Intensity;

	/** Inner of the region. Swapped with Outer in case it is higher than Outer. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Region", meta = (UIMin = 0.0, UIMax = 1.0))
	float Inner;

	/** Outer of the region. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Region", meta = (UIMin = 0.0, UIMax = 1.0))
	float Outer;

	/** Falloff. Softening the region. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Region", meta = (UIMin = 0.0))
	float Falloff;

	/** Invert region. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Region")
	bool Invert;

	/** Type of algorithm to be used to control color temperature or white balance. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Color Grading")
	EColorCorrectRegionTemperatureType TemperatureType;

	/** Color correction temperature. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Color Grading", meta = (UIMin = "1500.0", UIMax = "15000.0"))
	float Temperature;

	/** Color temperature tint. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Color Grading", meta=(UIMin = "-1.0", UIMax = "1.0"))
	float Tint;

	/** Color correction settings. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Color Grading", meta = (ShowOnlyInnerProperties))
	FColorGradingSettings ColorGradingSettings;

	/** Enable/Disable color correction provided by this region. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Color Grading", meta = (DisplayName = "Enable Color Grading", DisplayPriority = 1))
	bool Enabled;

	/** Enables or disabled per actor color correction. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Per Actor CC", meta = (DisplayName = "Enable Per-Actor CC"))
	bool bEnablePerActorCC;

	/** Controls in which way the below targets will be affected by color correction. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Per Actor CC", meta = (editcondition = "bEnablePerActorCC", DisplayName = "Per-Actor CC Mode"))
	EColorCorrectRegionStencilType PerActorColorCorrection;

	/** List of actors that get affected or ignored by Per actor CC. Effect depends on the above option. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Per Actor CC", meta = (editcondition = "bEnablePerActorCC", DisplayName = "Actor Selection"))
	TSet<TSoftObjectPtr<AActor>> AffectedActors;

	/** 
	* This is a temporary property that is used to keep track of managed actors for per actor CC.
	* PerAffectedActorStencilData is managed by FColorCorrectRegionsStencilManager and CCR Subsystem
	* Its lifetime is aligned with selected level and gets reset often.
	*/
	TMap<TSoftObjectPtr<AActor>, TSharedPtr<FStencilData>> PerAffectedActorStencilData;

#if WITH_EDITORONLY_DATA

	/** Billboard component for this actor. */
	UPROPERTY(Transient)
	TObjectPtr<UBillboardComponent> SpriteComponent;

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/** Called when any of the properties are changed. */
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditMove(bool bFinished) override;
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif

public:
	/** The main purpose of this component is to determine the visibility status of this Color Correction Actor. */
	UPROPERTY()
	TObjectPtr<UColorCorrectionInvisibleComponent> IdentityComponent;

	virtual bool ShouldTickIfViewportsOnly() const;

public:

	/**
	* Gets a full state for rendering.
	*/
	UE_DEPRECATED(5.5, "State management is now done by subsystem.")
	FColorCorrectRenderProxyPtr GetCCProxy_RenderThread() { return nullptr; };

	/**
	* Copy state required for rendering to be consumed by Scene view extension.
	*/
	UE_DEPRECATED(5.5, "State management is now done by subsystem.")
	void TransferState() {};

protected:

	/**
	* All CC Actors rely on a shape for selection. These shapes need to be swapped depending on the type.
	* This function forces the refresh of the shape if the Type was changed outside of UI.
	* For internal use only.
	*/
	virtual void ChangeShapeVisibilityForActorType() {};

	/** All CC Actors rely on a shape for selection. These shapes need to be swapped depending on the type. */
	template <typename TCCActorType>
	void ChangeShapeVisibilityForActorTypeInternal(TCCActorType InDesiredType)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!IsThisNotNull(this, "AColorCorrectRegion::ChangeShapeVisibilityForActorTypeInternal") || !IsValidChecked(this) || MeshComponents.Num() != (uint8)TCCActorType::MAX)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			return;
		}

		for (TCCActorType CCActorType : TEnumRange<TCCActorType>())
		{
			uint8 TypeIndex = static_cast<uint8>(CCActorType);

			if (!IsValid(MeshComponents[TypeIndex]))
			{
#if WITH_EDITOR
				FixMeshComponentReferencesInternal<TCCActorType>(InDesiredType);
#else
				ensure(IsValid(MeshComponents[TypeIndex]));
#endif
				return;
			}

			if (CCActorType == InDesiredType)
			{
				MeshComponents[TypeIndex]->SetVisibility(true, true);
			}
			else
			{
				MeshComponents[TypeIndex]->SetVisibility(false, true);
			}
		}
	};

#if WITH_EDITOR
protected:
	/** Used to validate child components. */
	virtual void FixMeshComponentReferences() {};

	template <typename TCCActorType>
	void FixMeshComponentReferencesInternal(TCCActorType InDesiredType)
	{
		if (!RootComponent)
		{
			return;
		}

		TArray<USceneComponent*> ChildComponents;
		RootComponent->GetChildrenComponents(false, ChildComponents);
		MeshComponents.Empty();
		for (TCCActorType CCActorType : TEnumRange<TCCActorType>())
		{
			const FString ShapeNameString = UEnum::GetValueAsString(CCActorType);
			for (USceneComponent* ChildComponent : ChildComponents)
			{
				TObjectPtr<UStaticMeshComponent> ChildMeshComponent = Cast<UStaticMeshComponent>(ChildComponent);
				if (ChildMeshComponent && ChildComponent->GetName() == ShapeNameString)
				{
					MeshComponents.Add(ChildMeshComponent);
					break;
				}

			}

		}
		ChangeShapeVisibilityForActorTypeInternal<TCCActorType>(InDesiredType);
	};
#endif

private:

	/**
	* AffectedActors property change could potentially invoke a Dialog Window, which should be displayed on Game Thread.
	* ActorListChangeType represents EPropertyChangeType
	*/
	void HandleAffectedActorsPropertyChange(uint32 ActorListChangeType);

#if WITH_EDITOR
	/** Called when a sequencer has its time changed. */
	void OnSequencerTimeChanged(TWeakPtr<class ISequencer> InSequencer);
#endif
	
public:

	// ~Begin IDisplayClusterStageActor interface
	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual void SetLongitude(double InValue) override;
	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual double GetLongitude() const override;

	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual void SetLatitude(double InValue) override;
	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual double GetLatitude() const override;

	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual void SetDistanceFromCenter(double InValue) override;
	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual double GetDistanceFromCenter() const override;

	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual void SetSpin(double InValue) override;
	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual double GetSpin() const override;

	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual void SetPitch(double InValue) override;
	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual double GetPitch() const override;

	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual void SetYaw(double InValue) override;
	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual double GetYaw() const override;

	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual void SetRadialOffset(double InValue) override;
	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual double GetRadialOffset() const override;

	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual void SetScale(const FVector2D& InScale) override;
	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual FVector2D GetScale() const override;

	UFUNCTION(BlueprintSetter)
	virtual void SetOrigin(const FTransform& InOrigin) override;
	UFUNCTION(BlueprintGetter)
	virtual FTransform GetOrigin() const override;

	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual void SetPositionalParams(const FDisplayClusterPositionalParams& InParams) override;
	UFUNCTION(BlueprintCallable, Category = Orientation)
	virtual FDisplayClusterPositionalParams GetPositionalParams() const override;

	virtual void GetPositionalProperties(FPositionalPropertyArray& OutPropertyPairs) const override;
	virtual FName GetPositionalPropertiesMemberName() const override;
	// ~End IDisplayClusterStageActor interface
	
protected:
	/** Spherical coordinates in relation to the origin, primarily used with the ICVFX panel. */
	UPROPERTY(EditAnywhere, Category = Orientation, meta = (ShowOnlyInnerProperties))
	FDisplayClusterPositionalParams PositionalParams;

	/** The origin when used in the ICVFX panel. */
	UPROPERTY(VisibleAnywhere, BlueprintSetter=SetOrigin, BlueprintGetter=GetOrigin, Category = Orientation)
	FTransform Origin;

	/** Update the transform when a positional setter is called. */
	bool bNotifyOnParamSetter = true;
};

/** 
 * NotPlaceable prohibits this actor from appearing in Place Actors menu by default. We add that manually with an icon attached.
*/
/**
 * A placeable Color Correction Regions actor that replaces previous implementation (blueprint). 
 * Color Correction Regions allow users to adjust color of anything that is within it (or outside, if Invert option is selected). 
 */
UCLASS(Blueprintable, NotPlaceable)
class COLORCORRECTREGIONS_API AColorCorrectionRegion : public AColorCorrectRegion
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	/** Called when any of the properties are changed. */
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual FName GetCustomIconName() const override;

protected:
	virtual void FixMeshComponentReferences() override;
#endif

protected:
	virtual void ChangeShapeVisibilityForActorType() override;

private:
#if WITH_METADATA
	/** Creates an icon for CCR/CCW to be clicked on in Editor. */
	void CreateIcon();

#endif // WITH_METADATA
};


/**
 * This component class is used to determine if Color Correction Window/Region is hidden via HiddenPrimitives/ShowOnlyPrimitivesShowOnlyPrimitives
 */
UCLASS(NotBlueprintable, NotPlaceable)
class UColorCorrectionInvisibleComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

};