// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(ColorCorrectRegions, Log, All);

class FColorCorrectRegionsModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation start */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	/** IModuleInterface implementation end */

#if WITH_EDITOR
	void RegisterOutlinerFilters();
#endif
};
