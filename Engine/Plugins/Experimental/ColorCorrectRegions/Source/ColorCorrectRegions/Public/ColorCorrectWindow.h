// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorCorrectRegion.h"
#include "Components/MeshComponent.h"
#include "CoreMinimal.h"
#include "Engine/Scene.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"

#include "ColorCorrectWindow.generated.h"


class UColorCorrectRegionsSubsystem;
class UBillboardComponent;

/**
 * A Color Correction Window that functions the same way as Color Correction Regions except it modifies anything that is behind it.
 * Color Correction Windows do not have Priority property and instead are sorted based on the distance from the camera.
 */
UCLASS(Blueprintable, notplaceable)
class COLORCORRECTREGIONS_API AColorCorrectionWindow : public AColorCorrectRegion
{
	GENERATED_UCLASS_BODY()
public:
	virtual ~AColorCorrectionWindow() override;

	/** Region type. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Region", Meta = (DisplayName = "Type"))
	EColorCorrectWindowType WindowType;

#if WITH_EDITOR
	/** Called when any of the properties are changed. */
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditMove(bool bFinished) override;
	virtual FName GetCustomIconName() const override;

protected:
	virtual void FixMeshComponentReferences() override;
#endif

protected:
	virtual void ChangeShapeVisibilityForActorType() override;

private:
#if WITH_METADATA
	void CreateIcon();
#endif // WITH_METADATA


};

UCLASS(Deprecated, Blueprintable, notplaceable, meta = (DeprecationMessage = "This is a deprecated version of Color Correct Window. Please re-create Color Correct Window to remove this warning."))
class COLORCORRECTREGIONS_API ADEPRECATED_ColorCorrectWindow : public AColorCorrectionWindow
{
	GENERATED_UCLASS_BODY()
};
