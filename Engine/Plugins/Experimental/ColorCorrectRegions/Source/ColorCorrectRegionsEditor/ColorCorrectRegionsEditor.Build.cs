// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ColorCorrectRegionsEditor : ModuleRules
{
	public ColorCorrectRegionsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ActorPickerMode",
				"Core",
				"CoreUObject",
				"Engine",
				"LevelEditor",
				"Projects",
				"UnrealEd",
				"Slate",
				"SlateCore",
				"ColorCorrectRegions",
				"ColorGradingEditor",
				"PlacementMode",
				"SceneOutliner",
				"ObjectMixerEditor",
				"ToolMenus",
				"TypedElementRuntime"
			}
		);
	}
}
