// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorCorrectRegionCustomization.h"
#include "ColorCorrectRegion.h"
#include "ColorCorrectWindow.h"

#include "ColorGradingEditorUtil.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "ObjectEditorUtils.h"

#define LOCTEXT_NAMESPACE "ColorCorrectWindowDetails"

TSharedRef<IDetailCustomization> FColorCorrectWindowDetails::MakeInstance()
{
	return MakeShareable(new FColorCorrectWindowDetails);
}

void FColorCorrectWindowDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Add hyphen to Per-Actor CC category. We have to do this here since by default, the editor will add a space after the hyphen
	const FText PerActorCCCategoryName = LOCTEXT("PerActorCCCategory", "Per-Actor CC");
	IDetailCategoryBuilder& PerActorCCCategory = DetailBuilder.EditCategory("Per Actor CC", PerActorCCCategoryName);
	PerActorCCCategory.SetDisplayName(PerActorCCCategoryName);

	// Hide CCR-specific properties if CCWs are present in the selection
	const bool bHasCCWs = DetailBuilder.GetSelectedObjects().ContainsByPredicate([](const TWeakObjectPtr<UObject>& SelectedObject)
	{
		return SelectedObject.IsValid() && SelectedObject->IsA<AColorCorrectionWindow>();
	});

	if (bHasCCWs)
	{
		TSharedRef<IPropertyHandle> PriorityProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, Priority));
		DetailBuilder.HideProperty(PriorityProperty);

		TSharedRef<IPropertyHandle> TypeProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, Type));
		DetailBuilder.HideProperty(TypeProperty);
	}

	// Create custom color grading category with button at top
	IDetailCategoryBuilder& ColorGradingCategory = DetailBuilder.EditCategory("Color Grading", LOCTEXT("ColorGradingCategory", "Color Grading"));
	ColorGradingCategory.AddCustomRow(LOCTEXT("OpenColorGrading", "Open Color Grading"))
		.RowTag("OpenColorGrading")
		[
			ColorGradingEditorUtil::MakeColorGradingLaunchButton()
		];

	// Place Enabled setting immediately under button
	ColorGradingCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, Enabled)));

	// Move temperature settings to custom group in custom Color Grading category
	{
		IDetailGroup& TemperatureGroup = ColorGradingCategory.AddGroup("Temperature", LOCTEXT("ColorGradingTemperatureGroup", "Temperature"));

		const TArray<FName> TemperaturePropertyNames = {
			GET_MEMBER_NAME_CHECKED(AColorCorrectionRegion, TemperatureType),
			GET_MEMBER_NAME_CHECKED(AColorCorrectionRegion, Temperature),
			GET_MEMBER_NAME_CHECKED(AColorCorrectionRegion, Tint)
		};

		for (const FName& PropertyName : TemperaturePropertyNames)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = DetailBuilder.GetProperty(PropertyName);
			TemperatureGroup.AddPropertyRow(ChildHandle.ToSharedRef());
		}
	}

	// Sort categories
	DetailBuilder.SortCategories([&](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
	{
		for (const TPair<FName, IDetailCategoryBuilder*>& Pair : CategoryMap)
		{
			int32 SortOrder = Pair.Value->GetSortOrder();
			const FName CategoryName = Pair.Key;
			const IDetailCategoryBuilder* Category = Pair.Value;

			if (CategoryName == "TransformCommon")
			{
				SortOrder = 0;
			}
			else if (CategoryName == "Region")
			{
				SortOrder = 1;
			}
			else if (Pair.Value == &ColorGradingCategory)
			{
				SortOrder = 2;
			}
			else if (Pair.Value == &PerActorCCCategory)
			{
				SortOrder = 3;
			}
			else if (CategoryName == "Orientation")
			{
				SortOrder = 4;
			}
			else
			{
				const int32 ValueSortOrder = Pair.Value->GetSortOrder();
				if (ValueSortOrder >= SortOrder && ValueSortOrder < SortOrder + 10)
				{
					SortOrder += 10;
				}
				else
				{
					continue;
				}
			}

			Pair.Value->SetSortOrder(SortOrder);
		}
	});
}

#undef LOCTEXT_NAMESPACE
