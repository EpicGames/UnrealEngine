// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"

class IDetailCategoryBuilder;

class FColorCorrectWindowDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};
