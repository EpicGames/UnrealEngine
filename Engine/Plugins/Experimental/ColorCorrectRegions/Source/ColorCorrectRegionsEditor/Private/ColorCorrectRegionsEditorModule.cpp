// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorCorrectRegionsEditorModule.h"
#include "ActorFactories/ActorFactoryBlueprint.h"
#include "ColorCorrectionActorContextMenu.h"
#include "ColorCorrectRegion.h"
#include "ColorCorrectRegionCustomization.h"
#include "ColorCorrectRegionsStyle.h"
#include "ColorCorrectWindow.h"
#include "ColorGradingDataModelGenerator_ColorCorrectRegion.h"
#include "ColorGradingEditorDataModel.h"
#include "ColorGradingHierarchyConfig_ColorCorrectRegion.h"
#include "ColorGradingMixerObjectFilterRegistry.h"
#include "IPlacementModeModule.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "FColorCorrectRegionsModule"

void FColorCorrectRegionsEditorModule::StartupModule()
{
	FColorCorrectRegionsStyle::Initialize();
	IPlacementModeModule::Get().OnPlacementModeCategoryRefreshed().AddRaw(this, &FColorCorrectRegionsEditorModule::OnPlacementModeRefresh);

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(AColorCorrectRegion::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FColorCorrectWindowDetails::MakeInstance));

	FColorGradingEditorDataModel::RegisterColorGradingDataModelGenerator<AColorCorrectRegion>(
		FGetDetailsDataModelGenerator::CreateStatic(&FColorGradingDataModelGenerator_ColorCorrectRegion::MakeInstance));

	FColorGradingMixerObjectFilterRegistry::RegisterActorClassToPlace(AColorCorrectionRegion::StaticClass());
	FColorGradingMixerObjectFilterRegistry::RegisterActorClassToPlace(AColorCorrectionWindow::StaticClass());

	FColorGradingMixerObjectFilterRegistry::RegisterObjectClassToFilter(
		AColorCorrectRegion::StaticClass(),
		FGetObjectHierarchyConfig::CreateStatic(&FColorGradingHierarchyConfig_ColorCorrectRegion::MakeInstance)
	);

	RegisterSectionMappings();

	ContextMenu = MakeShared<FColorCorrectionActorContextMenu>();
	ContextMenu->RegisterContextMenuExtender();
}

void FColorCorrectRegionsEditorModule::RegisterSectionMappings()
{
	static const FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

	const TArray<FName> ClassNames = {
		AColorCorrectionRegion::StaticClass()->GetFName(),
		AColorCorrectionWindow::StaticClass()->GetFName()
	};

	for (const FName& ClassName : ClassNames)
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(ClassName, "Color Grading", LOCTEXT("DetailsSectionColor Grading", "Color Grading"));
			Section->AddCategory("Color Grading");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(ClassName, "Orientation", LOCTEXT("DetailsSectionOrientation", "Orientation"));
			Section->AddCategory("Orientation");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(ClassName, "Per Actor CC", LOCTEXT("DetailsSectionPerActorCC", "Per Actor CC"));
			Section->AddCategory("Per Actor CC");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(ClassName, "Region", LOCTEXT("DetailsSectionRegion", "Region"));
			Section->AddCategory("Region");
		}
	}
}

void FColorCorrectRegionsEditorModule::OnPlacementModeRefresh(FName CategoryName)
{
	static FName VolumeName = FName(TEXT("Volumes"));
	static FName AllClasses = FName(TEXT("AllClasses"));

	if (CategoryName == VolumeName || CategoryName == AllClasses)
	{
		IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
		
		FPlaceableItem* CCRPlaceableItem = new FPlaceableItem(
			*AColorCorrectionRegion::StaticClass(),
			FAssetData(AColorCorrectionRegion::StaticClass(), true),
			FName("CCR.PlaceActorThumbnail"),
			FName("CCR.OutlinerThumbnail"),
			TOptional<FLinearColor>(),
			TOptional<int32>(),
			NSLOCTEXT("PlacementMode", "Color Correction Region", "Color Correction Region")
		);

		FPlaceableItem* CCWPlaceableItem = new FPlaceableItem(
			*AColorCorrectionWindow::StaticClass(),
			FAssetData(AColorCorrectionWindow::StaticClass()),
			FName("CCW.PlaceActorThumbnail"),
			FName("CCW.OutlinerThumbnail"),
			TOptional<FLinearColor>(),
			TOptional<int32>(), 
			NSLOCTEXT("PlacementMode", "Color Correction Window", "Color Correction Window"));

		PlacementModeModule.RegisterPlaceableItem(CategoryName, MakeShareable(CCWPlaceableItem));
		PlacementModeModule.RegisterPlaceableItem(CategoryName, MakeShareable(CCRPlaceableItem));
	}
}

void FColorCorrectRegionsEditorModule::ShutdownModule()
{
	FColorCorrectRegionsStyle::Shutdown();
	ContextMenu->UnregisterContextMenuExtender();
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FColorCorrectRegionsEditorModule, ColorCorrectRegionsEditor);
DEFINE_LOG_CATEGORY(ColorCorrectRegionsEditorLogOutput);
