// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorCorrectRegionsStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"

TSharedPtr<FSlateStyleSet> FColorCorrectRegionsStyle::CCRStyle;

void FColorCorrectRegionsStyle::Initialize()
{
	if (CCRStyle.IsValid())
	{
		return;
	}

	CCRStyle = MakeShared<FSlateStyleSet>(FName("ColorCorrectRegionsStyle"));

	FString CCR_IconPath = IPluginManager::Get().FindPlugin(TEXT("ColorCorrectRegions"))->GetBaseDir() + TEXT("/Resources/Template_CCR_64.svg");
	FString CCW_IconPath = IPluginManager::Get().FindPlugin(TEXT("ColorCorrectRegions"))->GetBaseDir() + TEXT("/Resources/Template_CCW_64.svg");

	CCRStyle->Set("CCR.PlaceActorThumbnail", new FSlateVectorImageBrush(CCR_IconPath, FVector2D(40.0f, 40.0f)));
	CCRStyle->Set("CCW.PlaceActorThumbnail", new FSlateVectorImageBrush(CCW_IconPath, FVector2D(40.0f, 40.0f)));

	CCRStyle->Set("CCR.OutlinerThumbnail", new FSlateVectorImageBrush(CCR_IconPath, FVector2D(20.0f, 20.0f)));
	CCRStyle->Set("CCW.OutlinerThumbnail", new FSlateVectorImageBrush(CCW_IconPath, FVector2D(20.0f, 20.0f)));
	CCRStyle->Set("ClassIcon.ColorCorrectionWindow", new FSlateVectorImageBrush(CCW_IconPath, FVector2D(20.0f, 20.0f)));
	CCRStyle->Set("ClassIcon.ColorCorrectionRegion", new FSlateVectorImageBrush(CCR_IconPath, FVector2D(20.0f, 20.0f)));
	CCRStyle->Set("ClassThumbnail.ColorCorrectionWindow", new FSlateVectorImageBrush(CCW_IconPath, FVector2D(20.0f, 20.0f)));
	CCRStyle->Set("ClassThumbnail.ColorCorrectionRegion", new FSlateVectorImageBrush(CCR_IconPath, FVector2D(20.0f, 20.0f)));

	FSlateStyleRegistry::RegisterSlateStyle(*CCRStyle.Get());
}

void FColorCorrectRegionsStyle::Shutdown()
{
	if (CCRStyle.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*CCRStyle.Get());
		ensure(CCRStyle.IsUnique());
		CCRStyle.Reset();
	}
}