// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorCorrectionActorContextMenu.h"
#include "ActorPickerMode.h"
#include "ActorTreeItem.h"
#include "ColorCorrectRegion.h"
#include "ColorCorrectWindow.h"
#include "ColorGradingMixerContextObject.h"
#include "Kismet/GameplayStatics.h"
#include "LevelEditor.h"
#include "LevelEditorMenuContext.h"
#include "Editor/SceneOutliner/Public/ActorMode.h"
#include "Engine/Selection.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor/SceneOutliner/Public/SSceneOutliner.h"
#include "Widgets/Input/SButton.h"
#include "SSocketChooser.h"
#include "Runtime/Engine/Classes/GameFramework/WorldSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Views/Widgets/ObjectMixerEditorListMenuContext.h"

#define LOCTEXT_NAMESPACE "FColorCorrectRegionsContextMenu"

namespace
{
	enum ECCType
	{
		Window,
		Region
	};

	class FCCActorPickingMode : public FActorMode
	{
	public:
		FCCActorPickingMode(SSceneOutliner* InSceneOutliner, FOnSceneOutlinerItemPicked InOnCCActorPicked);
		virtual ~FCCActorPickingMode() {}

		/* Begin ISceneOutlinerMode Implementation */
		virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;
		virtual bool ShowViewButton() const override { return true; }
		virtual bool ShouldShowFolders() const { return false; }
		/* End ISceneOutlinerMode Implementation */
	protected:

		/** Callback for when CC Actor is selected. */
		FOnSceneOutlinerItemPicked OnCCActorPicked;
	};

	// Map from actor to parent CCRs from which to remove the actor
	using TPerActorCCRemovalMap = TMap<AActor*, TArray<AColorCorrectRegion*>>;

	/** Add Selected actors to the chosen CC Actors' Per actor CC. */
	static void AddActorsToPerActorCC(TArray<AActor*> InCCActors, TArray<AActor*> InAffectedActors)
	{
		FScopedTransaction Transaction(LOCTEXT("AddToPerActorCCTransaction", "Add actors to Per-Actor CC"));

		for (AActor* CCActor : InCCActors)
		{
			if (AColorCorrectRegion* CCActorPtr = Cast<AColorCorrectRegion>(CCActor))
			{
				// Forcing property change event. If we don't do this, the stencil doesn't update properly
				const FName AffectedActorsPropertyName = GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, AffectedActors);
				FProperty* AffectedActorsProperty = FindFieldChecked<FProperty>(AColorCorrectRegion::StaticClass(), AffectedActorsPropertyName);
				CCActorPtr->PreEditChange(AffectedActorsProperty);

				CCActorPtr->bEnablePerActorCC = true;

				for (AActor* SelectedActor : InAffectedActors)
				{
					TSoftObjectPtr<AActor> SelectedActorPtr(SelectedActor);
					CCActorPtr->AffectedActors.Add(SelectedActorPtr);
				}

				FPropertyChangedEvent PropertyEvent(AffectedActorsProperty);
				PropertyEvent.ChangeType = EPropertyChangeType::ArrayAdd;
				CCActorPtr->PostEditChangeProperty(PropertyEvent);
			}
		}
	}

	/** Remove selected actors from the CC Actors' Per actor CC. */
	static void RemoveActorsFromPerActorCC(TSharedPtr<TPerActorCCRemovalMap> RemovalMap)
	{
		FScopedTransaction Transaction(LOCTEXT("RemoveFromPerActorCCTransaction", "Remove actors from Per-Actor CC"));

		for (TPair<AActor*, TArray<AColorCorrectRegion*>> Pair : *RemovalMap)
		{
			AActor* ActorToRemove = Pair.Key;
			TArray<AColorCorrectRegion*>& CCActors = Pair.Value;

			for (AColorCorrectRegion* CCActor : CCActors)
			{
				// Forcing property change event. If we don't do this, the stencil doesn't update properly
				const FName AffectedActorsPropertyName = GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, AffectedActors);
				FProperty* AffectedActorsProperty = FindFieldChecked<FProperty>(AColorCorrectRegion::StaticClass(), AffectedActorsPropertyName);
				CCActor->PreEditChange(AffectedActorsProperty);

				CCActor->AffectedActors.Remove(ActorToRemove);

				// If no affected actors are left, disable per-actor CC so it won't become invisible
				if (CCActor->AffectedActors.IsEmpty())
				{
					CCActor->bEnablePerActorCC = false;
				}

				FPropertyChangedEvent PropertyEvent(AffectedActorsProperty);
				PropertyEvent.ChangeType = EPropertyChangeType::ArrayRemove;
				CCActor->PostEditChangeProperty(PropertyEvent);
			}
		}
	}

	/** Creates a new CCR or CCW and then adds Selected actors to Per Actor CC. */
	static void CreateNewCCActor(ECCType InType, TSharedPtr<TArray<AActor*>> SelectedActors)
	{
		if (!SelectedActors.IsValid() || SelectedActors->Num() == 0)
		{
			return;
		}

		// Get bounds for the entire book 
		FVector Origin;
		FVector BoxExtent;
		UGameplayStatics::GetActorArrayBounds(*SelectedActors, false /*bOnlyCollidingComponents*/, Origin, BoxExtent);
		UWorld* World = (*SelectedActors)[0]->GetWorld();

		if (!World)
		{
			return;
		}

		AWorldSettings* WorldSettings = World->GetWorldSettings();

		if (!WorldSettings)
		{
			return;
		}

		TObjectPtr<AColorCorrectRegion> CCActorPtr;
		FTransform Transform;
		const FVector Scale = BoxExtent / (WorldSettings->WorldToMeters / 2.);

		// Adding a 1% scale offset for a better ecompassing of selected actors.
		Transform.SetScale3D(Scale*1.01);
		Transform.SetLocation(Origin);

		if (InType == ECCType::Window)
		{
			CCActorPtr = World->SpawnActor<AColorCorrectionWindow>();
		}
		else
		{
			AColorCorrectionRegion* CCActorRawPtr = World->SpawnActor<AColorCorrectionRegion>();
			CCActorRawPtr->Type = EColorCorrectRegionsType::Box;
			CCActorPtr = CCActorRawPtr;

			FPropertyChangedEvent TypeChangedEvent(AColorCorrectRegion::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, Type)));
			CCActorPtr->PostEditChangeProperty(TypeChangedEvent);
		}

		CCActorPtr->SetActorTransform(Transform);
		AddActorsToPerActorCC({ CCActorPtr }, *SelectedActors);

		// Hide all context menus.
		FSlateApplication::Get().DismissAllMenus();

		// Shift selection to the newly created CC Actor.
		GEditor->SelectNone(false/*bNoteSelectionChange*/, true/*bDeselectBSPSurfs*/);
		GEditor->SelectActor(CCActorPtr, true/*bInSelected*/, true/*bNotify*/);
	};

	/** Called by outliner when a new affected actor is selected to be added to one or more CC actors */
	static void OnAddPerActorCCFromCCRTreeItemSelected(TSharedRef<ISceneOutlinerTreeItem> NewAffectedActor, TSharedPtr<TArray<AActor*>> CCActors)
	{
		if (FActorTreeItem* ActorItem = NewAffectedActor->CastTo<FActorTreeItem>())
		{
			AddActorsToPerActorCC(*CCActors, { ActorItem->Actor.Get() });
		}

		// Hide all context menus.
		FSlateApplication::Get().DismissAllMenus();
	};

	/** Called by outliner when a CC actor is selected to have the affected actors added to its Per-Actor CC array */
	static void OnAddPerActorCCFromActorTreeItemSelected(TSharedRef<ISceneOutlinerTreeItem> CCActor, TSharedPtr<TArray<AActor*>> AffectedActors)
	{
		if (FActorTreeItem* ActorItem = CCActor->CastTo<FActorTreeItem>())
		{
			AddActorsToPerActorCC({ ActorItem->Actor.Get()}, *AffectedActors);
		}

		// Hide all context menus.
		FSlateApplication::Get().DismissAllMenus();
	};

	/** A helper function used to determine if selected actor is either CCR or CCW. */
	static bool IsActorCCR(const AActor* const InActor)
	{
		return (Cast<AColorCorrectRegion>(InActor) != nullptr);
	}

	/** Returns true if this actor can be added to any of the selected actors' per-actor CC lists. */
	static bool CanActorBeAddedToPerActorCC(const AActor* const InActor)
	{
		if (IsActorCCR(InActor))
		{
			return false;
		}

		return InActor->GetComponentByClass<UPrimitiveComponent>() != nullptr;
	}

	/** Transfers Editor into a picker state for selecting a color correction actor. */
	static FReply PickCCActorMode(TSharedPtr<TArray<AActor*>> InSelectedActors)
	{
		FSlateApplication::Get().DismissAllMenus();
		FActorPickerModeModule& ActorPickerModeModule = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");

		ActorPickerModeModule.BeginActorPickingMode(
			FOnGetAllowedClasses(),
			FOnShouldFilterActor::CreateStatic(IsActorCCR),
			FOnActorSelected::CreateLambda([InSelectedActors](AActor* InCCActor)
				{
					AddActorsToPerActorCC({ InCCActor }, *InSelectedActors);
				})
		);
		return FReply::Handled();
	}

	/** Transfers Editor into a picker state for selecting an actor to add to a color correction actor's per-actor CC list. */
	static FReply PickActorToAddToCCMode(TSharedPtr<TArray<AActor*>> InCCActors)
	{
		FSlateApplication::Get().DismissAllMenus();
		FActorPickerModeModule& ActorPickerModeModule = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");

		ActorPickerModeModule.BeginActorPickingMode(
			FOnGetAllowedClasses(),
			FOnShouldFilterActor::CreateStatic(CanActorBeAddedToPerActorCC),
			FOnActorSelected::CreateLambda([InCCActors](AActor* InSelectedActor)
				{
					AddActorsToPerActorCC(*InCCActors, { InSelectedActor });
				})
		);
		return FReply::Handled();
	}

	/** Build the menu that adds an actor to CCRs when right-clicking on the CCRs */
	static void CreateAddPerActorCCFromCCRMenu(UToolMenu* Menu, TSharedPtr<TArray<AActor*>> InSelectedActors)
	{
		FCreateSceneOutlinerMode ModeFactory = FCreateSceneOutlinerMode::CreateLambda([](SSceneOutliner* Outliner, TSharedPtr<TArray<AActor*>> SelectedActors)
		{
			return new FCCActorPickingMode(Outliner, FOnSceneOutlinerItemPicked::CreateStatic(OnAddPerActorCCFromCCRTreeItemSelected, SelectedActors));
		}, InSelectedActors);

		FSceneOutlinerInitializationOptions TargetSceneOutlinerInitOptions;
		TargetSceneOutlinerInitOptions.bShowHeaderRow = false;
		TargetSceneOutlinerInitOptions.bFocusSearchBoxWhenOpened = true;
		TargetSceneOutlinerInitOptions.ModeFactory = ModeFactory;
		TargetSceneOutlinerInitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateStatic(CanActorBeAddedToPerActorCC));

		TSharedRef<SWidget> CCSceneOutliner =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SSceneOutliner, TargetSceneOutlinerInitOptions)
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Top)
			.AutoWidth()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 9.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ToolTipText(LOCTEXT("PickActorButtonLabel", "Pick an Actor"))
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.OnClicked(FOnClicked::CreateStatic(PickActorToAddToCCMode, InSelectedActors))
					.ForegroundColor(FSlateColor::UseForeground())
					.IsFocusable(false)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.EyeDropper"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			];
		
		FToolMenuSection& ExistingSection = Menu->AddSection("Existing", LOCTEXT("ExistingCCActorSection", "Existing"));
		ExistingSection.AddEntry(
			FToolMenuEntry::InitWidget("Picker", CCSceneOutliner, FText::GetEmpty(), /*bNoIndent=*/ true)
		);
	};

	/** Build the menu that adds actors to a CCR's per-actor CC when right-clicking on an actor */
	static void CreateAddPerActorCCFromActorMenu(UToolMenu* Menu, TSharedPtr<TArray<AActor*>> InSelectedActors)
	{
		FToolMenuSection& TopSection = Menu->AddSection(NAME_None);

		TopSection.AddMenuEntry(
			"CreateAttachCCW",
			LOCTEXT("MenuCreateAttachCCW", "Add to New Color Correction Window"),
			LOCTEXT("MenuCreateAttachCCW_Tooltip", "Creates new Color Correction Window (CCW) and adds valid selected actors to Per-Actor CC of the newly created CCW."),
			FSlateIcon(),
			FExecuteAction::CreateStatic(CreateNewCCActor, Window, InSelectedActors),
			EUserInterfaceActionType::Button
		);

		TopSection.AddMenuEntry(
			"CreateAttachCCR",
			LOCTEXT("MenuCreateAttachCCR", "Add to New Color Correction Region"),
			LOCTEXT("MenuCreateAttachCCR_Tooltip", "Creates new Color Correction Region (CCR) and adds valid selected actors to Per-Actor CC of the newly created CCR."),
			FSlateIcon(),
			FExecuteAction::CreateStatic(CreateNewCCActor, Region, InSelectedActors),
			EUserInterfaceActionType::Button
		);

		FCreateSceneOutlinerMode ModeFactory = FCreateSceneOutlinerMode::CreateLambda([](SSceneOutliner* Outliner, TSharedPtr<TArray<AActor*>> SelectedActors)
		{
			return new FCCActorPickingMode(Outliner, FOnSceneOutlinerItemPicked::CreateStatic(OnAddPerActorCCFromActorTreeItemSelected, SelectedActors));
		}, InSelectedActors);
		
		FSceneOutlinerInitializationOptions CCSceneOutlinerInitOptions;
		CCSceneOutlinerInitOptions.bShowHeaderRow = false;
		CCSceneOutlinerInitOptions.bFocusSearchBoxWhenOpened = true;
		CCSceneOutlinerInitOptions.ModeFactory = ModeFactory;
		CCSceneOutlinerInitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateStatic(IsActorCCR));
		
		TSharedRef<SWidget> CCSceneOutliner =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SSceneOutliner, CCSceneOutlinerInitOptions)
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			]
		
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Top)
			.AutoWidth()
			[
				SNew(SVerticalBox)
		
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 9.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ToolTipText(LOCTEXT("PickCCActorButtonLabel", "Pick a Color Correction Actor"))
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.OnClicked(FOnClicked::CreateStatic(PickCCActorMode, InSelectedActors))
					.ForegroundColor(FSlateColor::UseForeground())
					.IsFocusable(false)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.EyeDropper"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			];


		FToolMenuSection& ExistingSection = Menu->AddSection("Existing", LOCTEXT("ExistingCCActorSection", "Existing"));
		ExistingSection.AddEntry(
			FToolMenuEntry::InitWidget("Picker", CCSceneOutliner, FText::GetEmpty(), /*bNoIndent=*/ true)
		);
	};

	/**
	 * Add the "Add to Per-Actor CC" menu entry, which either:
	 * 1) When CCRs are selected, adds an actor from a picker menu to their per-actor CC; or
	 * 2) When primitive actors are selected, adds them to the per-actor CC of a CCR from a picker menu
	 */
	static void AddAddToPerActorCCMenuEntry(FToolMenuSection& InSection, TSharedPtr<TArray<AActor*>> InSelectedActors)
	{
		const bool bIsCCROnlySelection = !InSelectedActors->ContainsByPredicate([](AActor* Actor)
		{
			return !IsActorCCR(Actor);
		});

		const bool bSelectionHasValidCCTargets = InSelectedActors->ContainsByPredicate(CanActorBeAddedToPerActorCC);

		const FText EntryLabel = LOCTEXT("AddToPerActorCCEntryName", "Add to Per-Actor CC");

		if (bIsCCROnlySelection)
		{
			InSection.AddSubMenu(
				TEXT("AddToPerActorCCEntry_CCR"),
				EntryLabel,
				LOCTEXT("AddToPerActorCCEntryTooltip_CCR", "Add actor selected in the following menu to the list of actors that get affected by the currently selected CC Actors."),
				FNewToolMenuDelegate::CreateStatic(CreateAddPerActorCCFromCCRMenu, InSelectedActors)
			);
		}
		else if (bSelectionHasValidCCTargets)
		{
			InSection.AddSubMenu(
				TEXT("AddToPerActorCCEntry_Actor"),
				EntryLabel,
				LOCTEXT("AddToPerActorCCEntryTooltip_Actor", "Add currently selected actors to the list of actors that get affected by selected CC Actor in the following menu."),
				FNewToolMenuDelegate::CreateStatic(CreateAddPerActorCCFromActorMenu, InSelectedActors)
			);
		}
	}

	/**
	 * Add the "Remove Per-Actor CC" menu entry where appropriate
	 */
	static void AddRemoveFromPerActorCCMenuEntry(UToolMenu* InMenu, FToolMenuSection& InSection, TSharedPtr<TArray<AActor*>> InSelectedActors)
	{
		// This is only relevant when clicking in the Color Grading hierarchy
		if (!InMenu->FindContext<UColorGradingMixerContextObject>())
		{
			return;
		}

		// This is also only relevant if the actor has a parent CCR in the hierarchy, and the parent affects the actor
		UObjectMixerEditorListMenuContext* ObjectMixerContext = InMenu->FindContext<UObjectMixerEditorListMenuContext>();
		if (!ObjectMixerContext)
		{
			return;
		}

		// Build a map from selected actor to CCRs from which the actor should be removed as an affected actor
		TPerActorCCRemovalMap RemovalMap;

		for (const TSharedPtr<ISceneOutlinerTreeItem>& TreeItem : ObjectMixerContext->Data.SelectedItems)
		{
			if (!TreeItem.IsValid() || !TreeItem->GetParent().IsValid())
			{
				continue;
			}

			if (FActorTreeItem* ParentActorItem = TreeItem->GetParent()->CastTo<FActorTreeItem>())
			{
				if (AColorCorrectRegion* ParentCCR = Cast<AColorCorrectRegion>(ParentActorItem->Actor))
				{
					for (AActor* SelectedActor : *InSelectedActors)
					{
						if (ParentCCR->AffectedActors.Contains(SelectedActor))
						{
							RemovalMap.FindOrAdd(SelectedActor).Add(ParentCCR);
						}
					}
				}
			}
		}

		if (RemovalMap.IsEmpty())
		{
			return;
		}

		const TSharedPtr<TPerActorCCRemovalMap> SharedRemovalMap = MakeShared<TPerActorCCRemovalMap>(MoveTemp(RemovalMap));

		InSection.AddMenuEntry(
			"MenuRemoveFromPerActorCC",
			LOCTEXT("MenuRemoveFromPerActorCC", "Remove from Per-Actor CC"),
			LOCTEXT("MenuRemoveFromPerActorCC_Tooltip", "Removes selected actors from the Per-Actor CC of the selection's parent CC Actors."),
			FSlateIcon(),
			FExecuteAction::CreateStatic(RemoveActorsFromPerActorCC, SharedRemovalMap),
			EUserInterfaceActionType::Button
		);
	}

	/** Adds the CCR section of a tool menu */
	static void AddColorCorrectRegionsSection(UToolMenu* InMenu, TSharedPtr<TArray<AActor*>> InSelectedActors)
	{
		FToolMenuSection& Section = InMenu->FindOrAddSection(
			"ColorCorrectionRegionsSection",
			LOCTEXT("ColorCorrectionRegions", "Color Correction Regions"),
			FToolMenuInsert("ActorTypeTools", EToolMenuInsertType::After)
		);

		AddAddToPerActorCCMenuEntry(Section, InSelectedActors);
		AddRemoveFromPerActorCCMenuEntry(InMenu, Section, InSelectedActors);
	}
}

void FColorCorrectionActorContextMenu::RegisterContextMenuExtender()
{
	if (UToolMenu* ActorContextMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.ActorContextMenu"))
	{
		FToolMenuSection& Section = ActorContextMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InMenu)
			{
				if (ULevelEditorContextMenuContext* Context = InMenu->FindContext<ULevelEditorContextMenuContext>())
				{
					TArray<AActor*> SelectedActors;

					if (Context->CurrentSelection)
					{
						Context->CurrentSelection->ForEachSelectedObject([&SelectedActors](UObject* Object)
						{
							if (AActor* Actor = Cast<AActor>(Object))
							{
								SelectedActors.Add(Actor);
							}
							return true;
						});
					}

					AddColorCorrectRegionsSection(InMenu, MakeShared<TArray<AActor*>>(MoveTemp(SelectedActors)));
				}
			}),
			FToolMenuInsert("ColorCorrectionRegionsSection", EToolMenuInsertType::After)
		);
	}
}

void FColorCorrectionActorContextMenu::UnregisterContextMenuExtender()
{
	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.GetAllLevelViewportContextMenuExtenders().RemoveAll(
			[&](const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& Delegate)
			{
				return Delegate.GetHandle() == ContextMenuExtenderDelegateHandle;
			});
	}
}

FCCActorPickingMode::FCCActorPickingMode(SSceneOutliner* InSceneOutliner, FOnSceneOutlinerItemPicked InOnCCActorPicked)
	: FActorMode(FActorModeParams(InSceneOutliner))
	, OnCCActorPicked(InOnCCActorPicked)
{
}

void FCCActorPickingMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	auto SelectedItems = SceneOutliner->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		auto FirstItem = SelectedItems[0];
		if (FirstItem->CanInteract())
		{
			OnCCActorPicked.ExecuteIfBound(FirstItem.ToSharedRef());
		}
	}
}

#undef LOCTEXT_NAMESPACE
