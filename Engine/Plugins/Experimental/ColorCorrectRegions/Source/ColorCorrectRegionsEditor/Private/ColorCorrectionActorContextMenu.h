// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorMode.h"
#include "LevelEditor.h"

class FColorCorrectionActorContextMenu
{
public:
	/** Create all required callbacks for context menu's widget and button creation. */
	void RegisterContextMenuExtender();

	/** Release context menu related resources such as handles. */ 
	void UnregisterContextMenuExtender();

private:
	/** Context menu extender. */
	FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors CCRLevelViewportContextMenuExtender;

	/** Handle to context menu extender delegate. */
	FDelegateHandle ContextMenuExtenderDelegateHandle;
};
