// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingDataModelGenerator_ColorCorrectRegion.h"

#include "ClassIconFinder.h"
#include "ColorCorrectRegion.h"
#include "ColorCorrectRegionCustomization.h"
#include "ColorCorrectWindow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "PropertyHandle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FColorGradingDataModelGenerator_ColorCorrectRegion"

const TArray<FName> VisibleCategories = {
	TEXT("Region"),
	TEXT("Color Grading"),
	TEXT("PerActorColorCorrection")
};

TSharedRef<IColorGradingEditorDataModelGenerator> FColorGradingDataModelGenerator_ColorCorrectRegion::MakeInstance()
{
	return MakeShareable(new FColorGradingDataModelGenerator_ColorCorrectRegion());
}

class FColorCorrectRegionCustomization : public IDetailCustomization
{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		// Hide any categories whose root isn't in our display list
		TArray<FName> Categories;
		DetailBuilder.GetCategoryNames(Categories);

		for (const FName& Category : Categories)
		{
			const FString RawCategoryName = Category.ToString();
			FName CategoryRootName;
			
			int32 SeparatorIndex;
			if (RawCategoryName.FindChar('|', SeparatorIndex))
			{
				CategoryRootName = FName(RawCategoryName.Left(SeparatorIndex));
			}
			else
			{
				CategoryRootName = Category;
			}

			if (!VisibleCategories.Contains(CategoryRootName))
			{
				DetailBuilder.HideCategory(Category);
			}
		}

		// TransformCommon is a custom category that doesn't get returned by GetCategoryNames that also needs to be hidden
		DetailBuilder.HideCategory(TEXT("TransformCommon"));

		IDetailCategoryBuilder& PerActorCCCategoryBuilder = DetailBuilder.EditCategory(TEXT("PerActorColorCorrection"), LOCTEXT("PerActorCCDisplayName", "Per-Actor Color Correction"));

		PerActorCCCategoryBuilder.AddProperty(DetailBuilder.GetProperty((GET_MEMBER_NAME_CHECKED(AColorCorrectionRegion, bEnablePerActorCC))));
		PerActorCCCategoryBuilder.AddProperty(DetailBuilder.GetProperty((GET_MEMBER_NAME_CHECKED(AColorCorrectionRegion, PerActorColorCorrection))));

		// Hide CCR-specific properties if CCWs are present in the selection
		const bool bHasCCWs = DetailBuilder.GetSelectedObjects().ContainsByPredicate([](const TWeakObjectPtr<UObject>& SelectedObject)
		{
			return SelectedObject.IsValid() && SelectedObject->IsA<AColorCorrectionWindow>();
		});

		if (bHasCCWs)
		{
			TSharedRef<IPropertyHandle> PriorityProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, Priority));
			DetailBuilder.HideProperty(PriorityProperty);

			TSharedRef<IPropertyHandle> TypeProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, Type));
			DetailBuilder.HideProperty(TypeProperty);
		}

		IDetailCategoryBuilder& ColorGradingElementsCategory = DetailBuilder.EditCategory(FName("ColorGradingElements"));

		ColorGradingElementsCategory.AddProperty(DetailBuilder.GetProperty((GET_MEMBER_NAME_CHECKED(AColorCorrectionRegion, ColorGradingSettings.Global))));
		ColorGradingElementsCategory.AddProperty(DetailBuilder.GetProperty((GET_MEMBER_NAME_CHECKED(AColorCorrectionRegion, ColorGradingSettings.Shadows))));
		ColorGradingElementsCategory.AddProperty(DetailBuilder.GetProperty((GET_MEMBER_NAME_CHECKED(AColorCorrectionRegion, ColorGradingSettings.Midtones))));
		ColorGradingElementsCategory.AddProperty(DetailBuilder.GetProperty((GET_MEMBER_NAME_CHECKED(AColorCorrectionRegion, ColorGradingSettings.Highlights))));
	}
};

void FColorGradingDataModelGenerator_ColorCorrectRegion::Initialize(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->RegisterInstancedCustomPropertyLayout(AColorCorrectRegion::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([]
	{
		return MakeShared<FColorCorrectRegionCustomization>();
	}));
}

void FColorGradingDataModelGenerator_ColorCorrectRegion::Destroy(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->UnregisterInstancedCustomPropertyLayout(AColorCorrectRegion::StaticClass());
}

void FColorGradingDataModelGenerator_ColorCorrectRegion::GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel)
{
	TArray<TWeakObjectPtr<AColorCorrectRegion>> SelectedCCRs;
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGenerator.GetSelectedObjects();
	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid() && SelectedObject->IsA<AColorCorrectRegion>())
		{
			TWeakObjectPtr<AColorCorrectRegion> SelectedRootActor = CastChecked<AColorCorrectRegion>(SelectedObject.Get());
			SelectedCCRs.Add(SelectedRootActor);
		}
	}

	if (!SelectedCCRs.Num())
	{
		return;
	}

	const TArray<TSharedRef<IDetailTreeNode>>& RootNodes = PropertyRowGenerator.GetRootTreeNodes();

	const TSharedRef<IDetailTreeNode>* ColorGradingElementsPtr = RootNodes.FindByPredicate([](const TSharedRef<IDetailTreeNode>& Node)
	{
		return Node->GetNodeName() == TEXT("ColorGradingElements");
	});

	if (ColorGradingElementsPtr)
	{
		const TSharedRef<IDetailTreeNode> ColorGradingElements = *ColorGradingElementsPtr;
		FColorGradingEditorDataModel::FColorGradingGroup ColorGradingGroup;

		ColorGradingGroup.DetailsViewCategories.Append(VisibleCategories);

		TArray<TSharedRef<IDetailTreeNode>> ColorGradingPropertyNodes;
		ColorGradingElements->GetChildren(ColorGradingPropertyNodes);

		for (const TSharedRef<IDetailTreeNode>& PropertyNode : ColorGradingPropertyNodes)
		{
			TSharedPtr<IPropertyHandle> PropertyHandle = PropertyNode->CreatePropertyHandle();

			FColorGradingEditorDataModel::FColorGradingElement ColorGradingElement = CreateColorGradingElement(PropertyNode, FText::FromName(PropertyNode->GetNodeName()));
			ColorGradingGroup.ColorGradingElements.Add(ColorGradingElement);
		}

		ColorGradingGroup.GroupHeaderWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0, 1, 6, 1))
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(16)
				.HeightOverride(16)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FClassIconFinder::FindIconForActor(SelectedCCRs[0]))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([WeakActor = SelectedCCRs[0]]()
				{
					return WeakActor.IsValid()
						? FText::FromString(WeakActor->GetActorLabel())
						: FText::GetEmpty();
				}))
				.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
			];

		OutColorGradingDataModel.ColorGradingGroups.Add(ColorGradingGroup);
	}
}

FColorGradingEditorDataModel::FColorGradingElement FColorGradingDataModelGenerator_ColorCorrectRegion::CreateColorGradingElement(const TSharedRef<IDetailTreeNode>& GroupNode, FText ElementLabel)
{
	FColorGradingEditorDataModel::FColorGradingElement ColorGradingElement;
	ColorGradingElement.DisplayName = ElementLabel;

	TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
	GroupNode->GetChildren(ChildNodes);

	for (const TSharedRef<IDetailTreeNode>& ChildNode : ChildNodes)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = ChildNode->CreatePropertyHandle();
		if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
		{
			const FString ColorGradingModeString = PropertyHandle->GetProperty()->GetMetaData(TEXT("ColorGradingMode")).ToLower();

			if (!ColorGradingModeString.IsEmpty())
			{
				if (ColorGradingModeString.Compare(TEXT("saturation")) == 0)
				{
					ColorGradingElement.SaturationPropertyHandle = PropertyHandle;
				}
				else if (ColorGradingModeString.Compare(TEXT("contrast")) == 0)
				{
					ColorGradingElement.ContrastPropertyHandle = PropertyHandle;
				}
				else if (ColorGradingModeString.Compare(TEXT("gamma")) == 0)
				{
					ColorGradingElement.GammaPropertyHandle = PropertyHandle;
				}
				else if (ColorGradingModeString.Compare(TEXT("gain")) == 0)
				{
					ColorGradingElement.GainPropertyHandle = PropertyHandle;
				}
				else if (ColorGradingModeString.Compare(TEXT("offset")) == 0)
				{
					ColorGradingElement.OffsetPropertyHandle = PropertyHandle;
				}
			}
		}
	}

	return ColorGradingElement;
}

#undef LOCTEXT_NAMESPACE