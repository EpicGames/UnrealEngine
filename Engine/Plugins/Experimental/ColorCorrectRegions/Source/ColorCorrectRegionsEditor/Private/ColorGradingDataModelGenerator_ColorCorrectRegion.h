// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ColorGradingEditorDataModel.h"

class IDetailTreeNode;

/** Color Grading Data Model Generator for the AColorCorrectionRegion actor class */
class FColorGradingDataModelGenerator_ColorCorrectRegion : public IColorGradingEditorDataModelGenerator
{
public:
	static TSharedRef<IColorGradingEditorDataModelGenerator> MakeInstance();

	//~ IColorGradingDataModelGenerator interface
	virtual void Initialize(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void Destroy(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel) override;
	//~ End IColorGradingDataModelGenerator interface

private:
	/** Creates a new color grading element structure for the specified detail tree node, which is expected to have child color properties with the ColorGradingMode metadata set */
	FColorGradingEditorDataModel::FColorGradingElement CreateColorGradingElement(const TSharedRef<IDetailTreeNode>& GroupNode, FText ElementLabel);
};