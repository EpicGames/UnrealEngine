// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingHierarchyConfig_ColorCorrectRegion.h"

#include "ActorTreeItem.h"
#include "ColorCorrectRegion.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "ColorGradingHierarchyConfig_ColorCorrectRegion"

TSharedRef<IColorGradingMixerObjectHierarchyConfig> FColorGradingHierarchyConfig_ColorCorrectRegion::MakeInstance()
{
	return MakeShared<FColorGradingHierarchyConfig_ColorCorrectRegion>();
}

TArray<AActor*> FColorGradingHierarchyConfig_ColorCorrectRegion::FindAssociatedActors(UObject* ParentObject) const
{
	TArray<AActor*> AssociatedActors;

	// List affected per-actor CC targets as associated actors so they appear as children in the hierarchy
	if (const AColorCorrectRegion* Region = Cast<AColorCorrectRegion>(ParentObject))
	{
		if (Region->bEnablePerActorCC)
		{
			for (TSoftObjectPtr<AActor> Actor : Region->AffectedActors)
			{
				if (Actor.IsValid())
				{
					AssociatedActors.Add(Actor.Get());
				}
			}
		}
	}

	return MoveTemp(AssociatedActors);
}

bool FColorGradingHierarchyConfig_ColorCorrectRegion::IsActorAssociated(UObject* ParentObject, AActor* AssociatedActor) const
{
	if (const AColorCorrectRegion* Region = Cast<AColorCorrectRegion>(ParentObject))
	{
		if (Region->bEnablePerActorCC)
		{
			return Region->AffectedActors.Contains(AssociatedActor);
		}
	}

	return false;
}

bool FColorGradingHierarchyConfig_ColorCorrectRegion::HasCustomDropHandling() const
{
	return true;
}

FSceneOutlinerDragValidationInfo FColorGradingHierarchyConfig_ColorCorrectRegion::ValidateDrop(UObject* DropTarget, const FSceneOutlinerDragDropPayload& Payload) const
{
	if (AColorCorrectRegion* Region = Cast<AColorCorrectRegion>(DropTarget))
	{
		if (Payload.SourceOperation.IsOfType<FActorDragDropOp>())
		{
			return FSceneOutlinerDragValidationInfo(
				ESceneOutlinerDropCompatibility::CompatibleAttach,
				FText::Format(
					LOCTEXT("FoldersOnActorError", "Add to Per-Actor CC for {0}"),
					FText::FromString(Region->GetActorLabel())
				)
			);
		}
	}

	return FSceneOutlinerDragValidationInfo::Invalid();
}

bool FColorGradingHierarchyConfig_ColorCorrectRegion::OnDrop(UObject* DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const
{
	if (AColorCorrectRegion* Region = Cast<AColorCorrectRegion>(DropTarget))
	{
		if (Payload.SourceOperation.IsOfType<FActorDragDropOp>())
		{
			GEditor->BeginTransaction(LOCTEXT("AddToPerActorCCTransaction", "Add actors to Per-Actor CC"));

			// Enable per-actor CC if not already enabled
			if (!Region->bEnablePerActorCC)
			{
				const FName EnablePerActorCCPropertyName = GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, bEnablePerActorCC);
				FProperty* EnablePerActorCCProperty = FindFieldChecked<FProperty>(AColorCorrectRegion::StaticClass(), EnablePerActorCCPropertyName);
				Region->PreEditChange(EnablePerActorCCProperty);

				Region->bEnablePerActorCC = true;

				FPropertyChangedEvent PropertyEvent(EnablePerActorCCProperty);
				PropertyEvent.ChangeType = EPropertyChangeType::ValueSet;
				Region->PostEditChangeProperty(PropertyEvent);
			}

			// Add affected actors from dragged list
			const FName AffectedActorsPropertyName = GET_MEMBER_NAME_CHECKED(AColorCorrectRegion, AffectedActors);
			FProperty* AffectedActorsProperty = FindFieldChecked<FProperty>(AColorCorrectRegion::StaticClass(), AffectedActorsPropertyName);
			Region->PreEditChange(AffectedActorsProperty);

			const FActorDragDropOp& ActorDragDropOp = StaticCast<const FActorDragDropOp&>(Payload.SourceOperation);
			for (TWeakObjectPtr<AActor> Actor : ActorDragDropOp.Actors)
			{
				if (Actor.IsValid())
				{
					Region->AffectedActors.Add(Actor.Get());
				}
			}

			FPropertyChangedEvent PropertyEvent(AffectedActorsProperty);
			PropertyEvent.ChangeType = EPropertyChangeType::ArrayAdd;
			Region->PostEditChangeProperty(PropertyEvent);

			GEditor->EndTransaction();

			return true;
		}
	}

	return false;
}

TSet<FName> FColorGradingHierarchyConfig_ColorCorrectRegion::GetPropertiesThatRequireListRefresh() const
{
	return {
		GET_MEMBER_NAME_CHECKED(AColorCorrectionRegion, bEnablePerActorCC),
		GET_MEMBER_NAME_CHECKED(AColorCorrectionRegion, AffectedActors)
	};
}

#undef LOCTEXT_NAMESPACE