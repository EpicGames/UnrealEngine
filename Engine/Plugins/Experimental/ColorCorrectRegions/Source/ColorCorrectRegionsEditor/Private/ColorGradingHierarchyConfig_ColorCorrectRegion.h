// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorGradingMixerObjectFilterRegistry.h"

/** Color Grading hierarchy config for the AColorCorrectionRegion actor class */
struct FColorGradingHierarchyConfig_ColorCorrectRegion : IColorGradingMixerObjectHierarchyConfig
{
public:
	static TSharedRef<IColorGradingMixerObjectHierarchyConfig> MakeInstance();

	//~ IColorGradingMixerObjectHierarchyConfig interface
	virtual TArray<AActor*> FindAssociatedActors(UObject* ParentObject) const override;
	virtual bool IsActorAssociated(UObject* ParentObject, AActor* AssociatedActor) const override;
	virtual bool HasCustomDropHandling() const override;
	virtual FSceneOutlinerDragValidationInfo ValidateDrop(UObject* DropTarget, const FSceneOutlinerDragDropPayload& Payload) const override;
	virtual bool OnDrop(UObject* DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const override;
	virtual TSet<FName> GetPropertiesThatRequireListRefresh() const override;
	//~ End IColorGradingMixerObjectHierarchyConfig interface
};