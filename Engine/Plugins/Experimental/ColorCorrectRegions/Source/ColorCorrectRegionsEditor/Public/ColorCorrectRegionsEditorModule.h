// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(ColorCorrectRegionsEditorLogOutput, Log, All);

class FColorCorrectRegionsEditorModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation start */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	/** IModuleInterface implementation end */

	void RegisterSectionMappings();
	void OnPlacementModeRefresh(FName CategoryName);

private:
	TSharedPtr<class FColorCorrectionActorContextMenu> ContextMenu;
};
